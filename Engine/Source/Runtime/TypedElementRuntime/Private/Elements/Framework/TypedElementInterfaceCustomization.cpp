// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementInterfaceCustomization.h"
#include "Elements/Framework/TypedElementRegistry.h"

FTypedHandleTypeId FTypedElementInterfaceCustomizationRegistryBase::GetElementTypeIdFromName(const FName InElementTypeName) const
{
	return UTypedElementRegistry::GetInstance()->GetRegisteredElementTypeId(InElementTypeName);
}

FTypedHandleTypeId FTypedElementInterfaceCustomizationRegistryBase::GetElementTypeIdFromNameChecked(const FName InElementTypeName) const
{
	const FTypedHandleTypeId ElementTypeId = GetElementTypeIdFromName(InElementTypeName);
	checkf(ElementTypeId > 0, TEXT("Element type '%s' has not been registered!"), *InElementTypeName.ToString());
	return ElementTypeId;
}
