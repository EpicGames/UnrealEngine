// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementListObjectUtil.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"

namespace TypedElementListObjectUtil
{

UObject* GetObjectOfType(const TTypedElement<ITypedElementObjectInterface>& InObjectElement, const UClass* InRequiredClass)
{
	UObject* ElementObject = InObjectElement ? InObjectElement.GetObject() : nullptr;
	return (ElementObject && (!InRequiredClass || ElementObject->IsA(InRequiredClass) || ElementObject->GetClass()->ImplementsInterface(InRequiredClass)))
		? ElementObject
		: nullptr;
}

bool HasObjects(FTypedElementListConstRef InElementList, const UClass* InRequiredClass)
{
	bool bHasObjects = false;

	ForEachObject(InElementList, [&bHasObjects](const UObject*)
	{
		bHasObjects = true;
		return false;
	}, InRequiredClass);

	return bHasObjects;
}

int32 CountObjects(FTypedElementListConstRef InElementList, const UClass* InRequiredClass)
{
	int32 NumObjects = 0;

	ForEachObject(InElementList, [&NumObjects](const UObject*)
	{
		++NumObjects;
		return true;
	}, InRequiredClass);

	return NumObjects;
}

void ForEachObject(FTypedElementListConstRef InElementList, TFunctionRef<bool(UObject*)> InCallback, const UClass* InRequiredClass)
{
	InElementList->ForEachElement<ITypedElementObjectInterface>([&InCallback, InRequiredClass](const TTypedElement<ITypedElementObjectInterface>& InObjectElement)
	{
		if (UObject* ElementObject = GetObjectOfType(InObjectElement, InRequiredClass))
		{
			return InCallback(ElementObject);
		}
		return true;
	});
}

TArray<UObject*> GetObjects(FTypedElementListConstRef InElementList, const UClass* InRequiredClass)
{
	TArray<UObject*> SelectedObjects;
	SelectedObjects.Reserve(InElementList->Num());

	ForEachObject(InElementList, [&SelectedObjects](UObject* InObject)
	{
		SelectedObjects.Add(InObject);
		return true;
	}, InRequiredClass);

	return SelectedObjects;
}

UObject* GetTopObject(FTypedElementListConstRef InElementList, const UClass* InRequiredClass)
{
	TTypedElement<ITypedElementObjectInterface> TempElement;
	for (int32 ElementIndex = 0; ElementIndex < InElementList->Num(); ++ElementIndex)
	{
		InElementList->GetElementAt(ElementIndex, TempElement);
		
		if (UObject* ElementObject = GetObjectOfType(TempElement, InRequiredClass))
		{
			return ElementObject;
		}
	}

	return nullptr;
}

UObject* GetBottomObject(FTypedElementListConstRef InElementList, const UClass* InRequiredClass)
{
	TTypedElement<ITypedElementObjectInterface> TempElement;
	for (int32 ElementIndex = InElementList->Num() - 1; ElementIndex >= 0; --ElementIndex)
	{
		InElementList->GetElementAt(ElementIndex, TempElement);

		if (UObject* ElementObject = GetObjectOfType(TempElement, InRequiredClass))
		{
			return ElementObject;
		}
	}

	return nullptr;
}

bool HasObjectsOfExactClass(FTypedElementListConstRef InElementList, const UClass* InClass)
{
	return CountObjectsOfExactClass(InElementList, InClass) > 0;
}

int32 CountObjectsOfExactClass(FTypedElementListConstRef InElementList, const UClass* InClass)
{
	return InElementList->GetCounter().GetCounterValue(NAME_Class, InClass);
}

void ForEachObjectClass(FTypedElementListConstRef InElementList, TFunctionRef<bool(UClass*)> InCallback)
{
	InElementList->GetCounter().ForEachCounterValue<UClass*>(NAME_Class, [&InCallback](UClass* InClass, FTypedElementCounter::FCounterValue InCount)
	{
		return InCallback(InClass);
	});
}

} // namespace TypedElementListObjectUtil
