// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Framework/TypedElementSelectionSet.h"

#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementUtil.h"
#include "Elements/Interfaces/TypedElementHierarchyInterface.h"
#include "HAL/IConsoleManager.h"

#include "Misc/ITransaction.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ObjectWriter.h"
#include "UObject/GCObjectScopeGuard.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementSelectionSet)

#if WITH_EDITORONLY_DATA
#include "Serialization/TextReferenceCollector.h"
#endif //WITH_EDITORONLY_DATA

namespace TypedElementSelectionSetLocals
{
	bool bResolveElementGetSelectionElementFirst = true;
	
	FAutoConsoleVariableRef CVarResolveElementGetSelectionElementFirst(
		TEXT("TypedElements.SelectionSet.ResolveElementGetSelectionElementFirst"),
		bResolveElementGetSelectionElementFirst,
		TEXT("If true UTypedElementSelectionSet::GetSelectionElement will allow the element "
			"to provide a selection element before running any customizations."));
}

void FTypedElementSelectionCustomization::GetNormalizedElements(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InSelectionSet, const FTypedElementSelectionNormalizationOptions& InNormalizationOptions, FTypedElementListRef OutNormalizedElements)
{
	// Don't include this element in the normalized selection if it has a parent element that is also selected
	{
		FTypedElementHandle ElementToTest = InElementSelectionHandle;
		while (TTypedElement<ITypedElementHierarchyInterface> ElementHierarchyHandle = InSelectionSet->GetElement<ITypedElementHierarchyInterface>(ElementToTest))
		{
			ElementToTest = ElementHierarchyHandle.GetParentElement();
			if (ElementToTest && InSelectionSet->Contains(ElementToTest))
			{
				return;
			}
		}
	}

	OutNormalizedElements->Add(InElementSelectionHandle);
}


UTypedElementSelectionSet::UTypedElementSelectionSet()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
		Registry->OnElementReplaced().AddUObject(this, &UTypedElementSelectionSet::OnElementReplaced);
		Registry->OnElementUpdated().AddUObject(this, &UTypedElementSelectionSet::OnElementUpdated);

		ElementList = Registry->CreateElementList();
		ElementList->OnPreChange().AddUObject(this, &UTypedElementSelectionSet::OnElementListPreChange);
		ElementList->OnChanged().AddUObject(this, &UTypedElementSelectionSet::OnElementListChanged);
	}
	
#if WITH_EDITORONLY_DATA
	// The selection set should not get text reference collected, as it is native and transient only, and should not find its way into asset packages
	{ static const FAutoRegisterTextReferenceCollectorCallback AutomaticRegistrationOfTextReferenceCollector(UTypedElementSelectionSet::StaticClass(), [](UObject* Object, FArchive& Ar) {}); }
#endif //WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
void UTypedElementSelectionSet::PreEditUndo()
{
	Super::PreEditUndo();

	checkf(!PendingUndoRedoState, TEXT("PendingUndoRedoState was set! Missing call to PostEditUndo?"));
	PendingUndoRedoState = MakeUnique<FTypedElementSelectionSetState>(GetCurrentSelectionState());

	FTypedElementSelectionSetState EmptySelectionState;
	EmptySelectionState.CreatedFromSelectionSet = this;
	RestoreSelectionState(EmptySelectionState);
}

void UTypedElementSelectionSet::PostEditUndo()
{
	Super::PostEditUndo();

	checkf(PendingUndoRedoState, TEXT("PendingUndoRedoState was null! Missing call to PreEditUndo?"));
	RestoreSelectionState(*PendingUndoRedoState);
	PendingUndoRedoState.Reset();
}

bool UTypedElementSelectionSet::Modify(bool bAlwaysMarkDirty)
{
	if (GUndo && CanModify() && !GUndo->ContainsObject(this))
	{
		bool bCanModify = true;
		ElementList->ForEachElement<ITypedElementSelectionInterface>([&bCanModify](const TTypedElement<ITypedElementSelectionInterface>& InSelectionElement)
		{
			bCanModify = !InSelectionElement.ShouldPreventTransactions();
			return bCanModify;
		});

		if (!bCanModify)
		{
			return false;
		}

		return Super::Modify(bAlwaysMarkDirty);
	}

	return false;
}
#endif	// WITH_EDITOR

void UTypedElementSelectionSet::BeginDestroy()
{
	if (ElementList)
	{
		ElementList->Empty();
	}

	Super::BeginDestroy();
}

void UTypedElementSelectionSet::Serialize(FArchive& Ar)
{
	checkf(!Ar.IsPersistent() || this->HasAnyFlags(RF_ClassDefaultObject),
		TEXT("UTypedElementSelectionSet can only be serialized by transient archives!"));

	const bool bIsUndoRedo = PendingUndoRedoState && Ar.IsTransacting();

	FTypedElementSelectionSetState TmpSelectionState;
	TmpSelectionState.CreatedFromSelectionSet = this;

	FTypedElementSelectionSetState& SelectionState = bIsUndoRedo ? *PendingUndoRedoState : TmpSelectionState;

	if (Ar.IsSaving())
	{
		if (ElementList && !bIsUndoRedo)
		{
			TmpSelectionState = GetCurrentSelectionState();
		}

		int32 NumTransactedElements = SelectionState.TransactedElements.Num();
		Ar << NumTransactedElements;

		for (const TUniquePtr<ITypedElementTransactedElement>& TransactedElement : SelectionState.TransactedElements)
		{
			FTypedHandleTypeId TypeId = TransactedElement->GetElementType();
			Ar << TypeId;

			TransactedElement->Serialize(Ar);
		}
	}
	else if (Ar.IsLoading())
	{
		UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();

		int32 NumTransactedElements = 0;
		Ar << NumTransactedElements;

		SelectionState.TransactedElements.Reset(NumTransactedElements);
		for (int32 TransactedElementIndex = 0; TransactedElementIndex < NumTransactedElements; ++TransactedElementIndex)
		{
			FTypedHandleTypeId TypeId = 0;
			Ar << TypeId;

			ITypedElementSelectionInterface* ElementTypeSelectionInterface = Registry->GetElementInterface<ITypedElementSelectionInterface>(TypeId);
			checkf(ElementTypeSelectionInterface, TEXT("Failed to find selection interface for a previously transacted element type!"));

			TUniquePtr<ITypedElementTransactedElement> TransactedElement = ElementTypeSelectionInterface->CreateTransactedElement(TypeId);
			checkf(TransactedElement, TEXT("Failed to allocate a transacted element for a previously transacted element type!"));

			TransactedElement->Serialize(Ar);
			SelectionState.TransactedElements.Emplace(MoveTemp(TransactedElement));
		}

		if (ElementList && !bIsUndoRedo)
		{
			RestoreSelectionState(SelectionState);
		}
	}
}

bool UTypedElementSelectionSet::IsElementSelected(const FTypedElementHandle& InElementHandle, const FTypedElementIsSelectedOptions InSelectionOptions) const
{
	FTypedElementSelectionSetElement SelectionSetElement = ResolveSelectionSetElement(InElementHandle);
	return SelectionSetElement && SelectionSetElement.IsElementSelected(InSelectionOptions);
}

bool UTypedElementSelectionSet::CanSelectElement(const FTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions InSelectionOptions) const
{
	FTypedElementSelectionSetElement SelectionSetElement = ResolveSelectionSetElement(InElementHandle);
	return SelectionSetElement && SelectionSetElement.CanSelectElement(InSelectionOptions);
}

bool UTypedElementSelectionSet::CanDeselectElement(const FTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions InSelectionOptions) const
{
	FTypedElementSelectionSetElement SelectionSetElement = ResolveSelectionSetElement(InElementHandle);
	return SelectionSetElement && SelectionSetElement.CanDeselectElement(InSelectionOptions);
}

bool UTypedElementSelectionSet::SelectElement(const FTypedElementHandle& InElementHandle, FTypedElementSelectionOptions InSelectionOptions)
{
	if (InSelectionOptions.GetNameForTEDSIntegration().IsNone())
	{
		InSelectionOptions.SetNameForTEDSIntegration(ListNameForTedsIntegration);
	}

	bool bSelectionChanged = false;

	if (FTypedElementSelectionSetElement SelectionSetElement = ResolveSelectionSetElement(InElementHandle))
	{
		if (SelectionSetElement.CanSelectElement(InSelectionOptions))
		{
			bSelectionChanged |= SelectionSetElement.SelectElement(InSelectionOptions);
		}
	}

	if (InSelectionOptions.GetChildElementInclusionMethod() != ETypedElementChildInclusionMethod::None)
	{
		if (TTypedElement<ITypedElementHierarchyInterface> ElementHierarchyHandle = ElementList->GetElement<ITypedElementHierarchyInterface>(InElementHandle))
		{
			const FTypedElementSelectionOptions ChildSelectionOptions = FTypedElementSelectionOptions(InSelectionOptions)
				.SetChildElementInclusionMethod(InSelectionOptions.GetChildElementInclusionMethod() == ETypedElementChildInclusionMethod::Immediate ? ETypedElementChildInclusionMethod::None : InSelectionOptions.GetChildElementInclusionMethod());

			TArray<FTypedElementHandle> ChildElementHandles;
			ElementHierarchyHandle.GetChildElements(ChildElementHandles, /*bAllowCreate*/true);
			for (const FTypedElementHandle& ChildElementHandle : ChildElementHandles)
			{
				bSelectionChanged |= SelectElement(ChildElementHandle, ChildSelectionOptions);
			}
		}
	}

	return bSelectionChanged;
}

bool UTypedElementSelectionSet::SelectElements(const TArray<FTypedElementHandle>& InElementHandles, const FTypedElementSelectionOptions InSelectionOptions)
{
	return SelectElements(MakeArrayView(InElementHandles), InSelectionOptions);
}

bool UTypedElementSelectionSet::SelectElements(TArrayView<const FTypedElementHandle> InElementHandles, const FTypedElementSelectionOptions InSelectionOptions)
{
	FTypedElementList::FLegacySyncScopedBatch LegacySyncBatch(*ElementList, InSelectionOptions.AllowLegacyNotifications());

	bool bSelectionChanged = false;
	ElementList->Reserve(ElementList->Num() + InElementHandles.Num());

	for (const FTypedElementHandle& ElementHandle : InElementHandles)
	{
		bSelectionChanged |= SelectElement(ElementHandle, InSelectionOptions);
	}

	return bSelectionChanged;
}

bool UTypedElementSelectionSet::SelectElements(FTypedElementListConstRef InElementList, const FTypedElementSelectionOptions InSelectionOptions)
{
	FTypedElementList::FLegacySyncScopedBatch LegacySyncBatch(*ElementList, InSelectionOptions.AllowLegacyNotifications());

	bool bSelectionChanged = false;
	ElementList->Reserve(ElementList->Num() + InElementList->Num());

	InElementList->ForEachElementHandle([this, &bSelectionChanged, &InSelectionOptions](const FTypedElementHandle& ElementHandle)
	{
		bSelectionChanged |= SelectElement(ElementHandle, InSelectionOptions);
		return true;
	});

	return bSelectionChanged;
}

bool UTypedElementSelectionSet::DeselectElement(const FTypedElementHandle& InElementHandle, FTypedElementSelectionOptions InSelectionOptions)
{
	if (InSelectionOptions.GetNameForTEDSIntegration().IsNone())
	{
		InSelectionOptions.SetNameForTEDSIntegration(ListNameForTedsIntegration);
	}

	bool bSelectionChanged = false;

	if (FTypedElementSelectionSetElement SelectionSetElement = ResolveSelectionSetElement(InElementHandle))
	{
		if (SelectionSetElement.CanDeselectElement(InSelectionOptions))
		{
			bSelectionChanged |= SelectionSetElement.DeselectElement(InSelectionOptions);
		}
	}

	if (InSelectionOptions.GetChildElementInclusionMethod() != ETypedElementChildInclusionMethod::None)
	{
		if (TTypedElement<ITypedElementHierarchyInterface> ElementHierarchyHandle = ElementList->GetElement<ITypedElementHierarchyInterface>(InElementHandle))
		{
			const FTypedElementSelectionOptions ChildSelectionOptions = FTypedElementSelectionOptions(InSelectionOptions)
				.SetChildElementInclusionMethod(InSelectionOptions.GetChildElementInclusionMethod() == ETypedElementChildInclusionMethod::Immediate ? ETypedElementChildInclusionMethod::None : InSelectionOptions.GetChildElementInclusionMethod());

			TArray<FTypedElementHandle> ChildElementHandles;
			ElementHierarchyHandle.GetChildElements(ChildElementHandles, /*bAllowCreate*/false);
			for (const FTypedElementHandle& ChildElementHandle : ChildElementHandles)
			{
				bSelectionChanged |= DeselectElement(ChildElementHandle, ChildSelectionOptions);
			}
		}
	}

	return bSelectionChanged;
}

bool UTypedElementSelectionSet::DeselectElements(const TArray<FTypedElementHandle>& InElementHandles, const FTypedElementSelectionOptions InSelectionOptions)
{
	return DeselectElements(MakeArrayView(InElementHandles), InSelectionOptions);
}

bool UTypedElementSelectionSet::DeselectElements(TArrayView<const FTypedElementHandle> InElementHandles, const FTypedElementSelectionOptions InSelectionOptions)
{
	FTypedElementList::FLegacySyncScopedBatch LegacySyncBatch(*ElementList, InSelectionOptions.AllowLegacyNotifications());

	bool bSelectionChanged = false;

	for (const FTypedElementHandle& ElementHandle : InElementHandles)
	{
		bSelectionChanged |= DeselectElement(ElementHandle, InSelectionOptions);
	}

	return bSelectionChanged;
}

bool UTypedElementSelectionSet::DeselectElements(FTypedElementListConstRef InElementList, const FTypedElementSelectionOptions InSelectionOptions)
{
	FTypedElementList::FLegacySyncScopedBatch LegacySyncBatch(*ElementList, InSelectionOptions.AllowLegacyNotifications());

	bool bSelectionChanged = false;

	InElementList->ForEachElementHandle([this, &bSelectionChanged, &InSelectionOptions](const FTypedElementHandle& ElementHandle)
	{
		bSelectionChanged |= DeselectElement(ElementHandle, InSelectionOptions);
		return true;
	});

	return bSelectionChanged;
}

bool UTypedElementSelectionSet::ClearSelection(const FTypedElementSelectionOptions InSelectionOptions)
{
	FTypedElementList::FLegacySyncScopedBatch LegacySyncBatch(*ElementList, InSelectionOptions.AllowLegacyNotifications());
	
	bool bSelectionChanged = false;

	// Run deselection via the selection interface where possible
	{
		// Take a copy of the currently selected elements to avoid mutating the selection set while iterating
		TArray<FTypedElementHandle, TInlineAllocator<8>> ElementsCopy;
		ElementList->GetElementHandles(ElementsCopy);
		FTypedElementSelectionOptions CopySelectionOptions(InSelectionOptions);

		// Always support SubRootSelection when clearing
		CopySelectionOptions.SetAllowSubRootSelection(true);
		for (const FTypedElementHandle& ElementHandle : ElementsCopy)
		{
			bSelectionChanged |= DeselectElement(ElementHandle, CopySelectionOptions);
		}
	}

	// TODO: BSP surfaces?

	// If anything remains in the selection set after processing elements that implement this interface, just clear it
	if (ElementList->Num() > 0)
	{
		bSelectionChanged = true;
		ElementList->Reset();
	}

	return bSelectionChanged;
}

bool UTypedElementSelectionSet::SetSelection(const TArray<FTypedElementHandle>& InElementHandles, const FTypedElementSelectionOptions InSelectionOptions)
{
	return SetSelection(MakeArrayView(InElementHandles), InSelectionOptions);
}

bool UTypedElementSelectionSet::SetSelection(TArrayView<const FTypedElementHandle> InElementHandles, const FTypedElementSelectionOptions InSelectionOptions)
{
	FTypedElementList::FLegacySyncScopedBatch LegacySyncBatch(*ElementList, InSelectionOptions.AllowLegacyNotifications());

	bool bSelectionChanged = false;

	bSelectionChanged |= ClearSelection(InSelectionOptions);
	bSelectionChanged |= SelectElements(InElementHandles, InSelectionOptions);

	return bSelectionChanged;
}

bool UTypedElementSelectionSet::SetSelection(FTypedElementListConstRef InElementList, const FTypedElementSelectionOptions InSelectionOptions)
{
	FTypedElementList::FLegacySyncScopedBatch LegacySyncBatch(*ElementList, InSelectionOptions.AllowLegacyNotifications());

	bool bSelectionChanged = false;

	bSelectionChanged |= ClearSelection(InSelectionOptions);
	bSelectionChanged |= SelectElements(InElementList, InSelectionOptions);

	return bSelectionChanged;
}

bool UTypedElementSelectionSet::AllowSelectionModifiers(const FTypedElementHandle& InElementHandle) const
{
	FTypedElementSelectionSetElement SelectionSetElement = ResolveSelectionSetElement(InElementHandle);
	return SelectionSetElement && SelectionSetElement.AllowSelectionModifiers();
}

void UTypedElementSelectionSet::SetNameForTedsIntegration(FName InNameForIntegration)
{
	ListNameForTedsIntegration = InNameForIntegration;
}

FName UTypedElementSelectionSet::GetNameForTedsIntegration() const
{
	return ListNameForTedsIntegration;
}

FTypedElementHandle UTypedElementSelectionSet::GetSelectionElement(const FTypedElementHandle& InElementHandle, const ETypedElementSelectionMethod InSelectionMethod) const
{
	FTypedElementHandle HandleToUse = InElementHandle;
	
	// Give the element interface a chance to remap to a different element first. This allows proxy elements to
	//  map to their backing element and then have the backing element be used for any selection customization.
	if (TypedElementSelectionSetLocals::bResolveElementGetSelectionElementFirst)
	{
		if (const TTypedElement<ITypedElementSelectionInterface> ElementSelectionInterface = GetElementList()->GetElement<ITypedElementSelectionInterface>(InElementHandle))
		{
			HandleToUse = ElementSelectionInterface.GetSelectionElement(GetElementList(), InSelectionMethod);
		}
	}

	FTypedElementSelectionSetElement SelectionSetElement = ResolveSelectionSetElement(HandleToUse);
	return SelectionSetElement ? SelectionSetElement.GetSelectionElement(InSelectionMethod) : FTypedElementHandle();
}

FTypedElementListRef UTypedElementSelectionSet::GetNormalizedSelection(const FTypedElementSelectionNormalizationOptions InNormalizationOptions) const
{
	return GetNormalizedElementList(ElementList.ToSharedRef(), InNormalizationOptions);
}

void UTypedElementSelectionSet::GetNormalizedSelection(const FTypedElementSelectionNormalizationOptions& InNormalizationOptions, FTypedElementListRef OutNormalizedElements) const
{
	GetNormalizedElementList(ElementList.ToSharedRef(), InNormalizationOptions, OutNormalizedElements);
}

FTypedElementListRef UTypedElementSelectionSet::GetNormalizedElementList(FTypedElementListConstRef InElementList, const FTypedElementSelectionNormalizationOptions InNormalizationOptions) const
{
	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
	FTypedElementListRef NormalizedElementList(Registry->CreateElementList());
	GetNormalizedElementList(InElementList, InNormalizationOptions, NormalizedElementList);
	return NormalizedElementList;
}

void UTypedElementSelectionSet::GetNormalizedElementList(FTypedElementListConstRef InElementList, const FTypedElementSelectionNormalizationOptions& InNormalizationOptions, FTypedElementListRef OutNormalizedElements) const
{
	OutNormalizedElements->Reset();
	InElementList->ForEachElement<ITypedElementSelectionInterface>([this, InElementList, &InNormalizationOptions, OutNormalizedElements](const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle)
	{
		// Note: FTypedElementSelectionSetElement needs a non-const FTypedElementList, but we won't call anything that will modify it...
		FTypedElementSelectionSetElement SelectionSetElement(InElementSelectionHandle, ConstCastSharedRef<FTypedElementList>(InElementList), GetInterfaceCustomizationByTypeId(InElementSelectionHandle.GetId().GetTypeId()));
		check(SelectionSetElement.IsSet());
		SelectionSetElement.GetNormalizedElements(InNormalizationOptions, OutNormalizedElements);
		return true;
	});
}

FTypedElementSelectionSetState UTypedElementSelectionSet::GetCurrentSelectionState() const
{
	FTypedElementSelectionSetState CurrentState;

	CurrentState.CreatedFromSelectionSet = this;

	CurrentState.TransactedElements.Reserve(ElementList->Num());
	ElementList->ForEachElement<ITypedElementSelectionInterface>([&CurrentState](const TTypedElement<ITypedElementSelectionInterface>& InSelectionElement)
	{
		if (TUniquePtr<ITypedElementTransactedElement> TransactedElement = InSelectionElement.CreateTransactedElement())
		{
			CurrentState.TransactedElements.Emplace(MoveTemp(TransactedElement));
		}
		return true;
	});

	return CurrentState;
}

void UTypedElementSelectionSet::RestoreSelectionState(const FTypedElementSelectionSetState& InSelectionState)
{
	if (InSelectionState.CreatedFromSelectionSet == this)
	{
		TArray<FTypedElementHandle, TInlineAllocator<256>> SelectedElements;
		SelectedElements.Reserve(InSelectionState.TransactedElements.Num());
		
		for (const TUniquePtr<ITypedElementTransactedElement>& TransactedElement : InSelectionState.TransactedElements)
		{
			if (FTypedElementHandle SelectedElement = TransactedElement->GetElement())
			{
				SelectedElements.Add(MoveTemp(SelectedElement));
			}
		}

		{
			TGuardValue<bool> GuardIsRestoringState(bIsRestoringState, true);

			const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
				.SetAllowHidden(true)
				.SetAllowGroups(false)
				.SetAllowLegacyNotifications(false)
				.SetWarnIfLocked(false)
				.SetNameForTEDSIntegration(ListNameForTedsIntegration);


			// TODO: Work out the intersection of the before and after state instead of clearing and reselecting?
			SetSelection(SelectedElements, SelectionOptions);
		}
	}
}

FTypedElementSelectionSetElement UTypedElementSelectionSet::ResolveSelectionSetElement(const FTypedElementHandle& InElementHandle) const
{
	return InElementHandle
		? FTypedElementSelectionSetElement(ElementList->GetElement<ITypedElementSelectionInterface>(InElementHandle), ElementList, GetInterfaceCustomizationByTypeId(InElementHandle.GetId().GetTypeId()))
		: FTypedElementSelectionSetElement();
}

void UTypedElementSelectionSet::OnElementReplaced(TArrayView<const TTuple<FTypedElementHandle, FTypedElementHandle>> InReplacedElements)
{
	const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
		.SetAllowHidden(true)
		.SetAllowGroups(false)
		.SetAllowLegacyNotifications(false)
		.SetWarnIfLocked(false);

	// We're updating to keep in sync with element data that may have been changed by other aspects of the engine
	// The selection isn't actually changing, so don't broadcast all the notifications about the select and deselects we do here.
	FTypedElementList::FScopedClearNewPendingChange ScopedClearChanges = GetScopedClearNewPendingChange();
	FTypedElementList::FLegacySyncScopedBatch LegacySync(*ElementList, SelectionOptions.AllowLegacyNotifications());

	for (const TTuple<FTypedElementHandle, FTypedElementHandle>& ReplacedElement : InReplacedElements)
	{
		if (ElementList->Num() == 0)
		{
			break;
		}

		bool bDeselectedElement = false;
		{
			FTypedElementSelectionSetElement SelectionSetElementToRemove = ResolveSelectionSetElement(ReplacedElement.Key);
			if (SelectionSetElementToRemove && SelectionSetElementToRemove.CanDeselectElement(SelectionOptions))
			{
				bDeselectedElement = SelectionSetElementToRemove.DeselectElement(SelectionOptions);
			}
			else
			{
				bDeselectedElement = ElementList->Remove(ReplacedElement.Key);
			}
		}

		if (bDeselectedElement && ReplacedElement.Value)
		{
			SelectElement(ReplacedElement.Value, SelectionOptions);
		}
	}
}

void UTypedElementSelectionSet::OnElementUpdated(TArrayView<const FTypedElementHandle> InUpdatedElements)
{
	const FTypedElementSelectionOptions SelectionOptions = FTypedElementSelectionOptions()
		.SetAllowHidden(true)
		.SetAllowGroups(false)
		.SetAllowLegacyNotifications(false)
		.SetWarnIfLocked(false);

	if (ElementList->Num() == 0)
	{
		return;
	}

	// We're updating to keep in sync with element data that may have been changed by other aspects of the engine
	// The selection isn't actually changing, so don't broadcast all the notifications about the select and deselects we do here.
	FTypedElementList::FScopedClearNewPendingChange ScopedClearChanges = GetScopedClearNewPendingChange();
	FTypedElementList::FLegacySyncScopedBatch LegacySync(*ElementList, SelectionOptions.AllowLegacyNotifications());

	for (const FTypedElementHandle& UpdatedElement : InUpdatedElements)
	{
		FTypedElementSelectionSetElement SelectionSetElementToUpdate = ResolveSelectionSetElement(UpdatedElement);

		bool bDeselectedElement = false;
		if (SelectionSetElementToUpdate && SelectionSetElementToUpdate.CanDeselectElement(SelectionOptions))
		{
			bDeselectedElement = SelectionSetElementToUpdate.DeselectElement(SelectionOptions);
		}
		else
		{
			bDeselectedElement = ElementList->Remove(UpdatedElement);
		}

		if (bDeselectedElement && SelectionSetElementToUpdate.CanSelectElement(SelectionOptions))
		{
			SelectionSetElementToUpdate.SelectElement(SelectionOptions);
		}
	}
}

void UTypedElementSelectionSet::OnElementListPreChange(const FTypedElementList& InElementList)
{
	check(&InElementList == ElementList.Get());
	OnPreChangeDelegate.Broadcast(this);
	OnPreSelectionChange.Broadcast(this);

	if (!bIsRestoringState)
	{
		// Track the pre-change state for undo/redo
		Modify();
	}
}

void UTypedElementSelectionSet::OnElementListChanged(const FTypedElementList& InElementList)
{
	check(&InElementList == ElementList.Get());
	OnChangedDelegate.Broadcast(this);
	OnSelectionChange.Broadcast(this);
}

bool UTypedElementSelectionSet::IsElementSelected(const FScriptTypedElementHandle& InElementHandle, const FTypedElementIsSelectedOptions InSelectionOptions) const
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return IsElementSelected(NativeHandle, InSelectionOptions);
}

bool UTypedElementSelectionSet::CanSelectElement(const FScriptTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions InSelectionOptions) const
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return CanSelectElement(NativeHandle, InSelectionOptions);
}

bool UTypedElementSelectionSet::CanDeselectElement(const FScriptTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions InSelectionOptions) const
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return CanDeselectElement(NativeHandle, InSelectionOptions);
}

bool UTypedElementSelectionSet::SelectElement(const FScriptTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions InSelectionOptions)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return SelectElement(NativeHandle, InSelectionOptions);
}

bool UTypedElementSelectionSet::SelectElements(const TArray<FScriptTypedElementHandle>& InElementHandles, const FTypedElementSelectionOptions InSelectionOptions)
{
	TArray<FTypedElementHandle> NativeHandles = TypedElementUtil::ConvertToNativeElementArray(InElementHandles);
	return SelectElements(NativeHandles, InSelectionOptions);
}


bool UTypedElementSelectionSet::DeselectElement(const FScriptTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions InSelectionOptions)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return DeselectElement(NativeHandle, InSelectionOptions);
}

bool UTypedElementSelectionSet::DeselectElements(const TArray<FScriptTypedElementHandle>& InElementHandles, const FTypedElementSelectionOptions InSelectionOptions)
{
	TArray<FTypedElementHandle> NativeHandles = TypedElementUtil::ConvertToNativeElementArray(InElementHandles);
	return DeselectElements(NativeHandles, InSelectionOptions);
}

bool UTypedElementSelectionSet::SetSelection(const TArray<FScriptTypedElementHandle>& InElementHandles, const FTypedElementSelectionOptions InSelectionOptions)
{
	TArray<FTypedElementHandle> NativeHandles = TypedElementUtil::ConvertToNativeElementArray(InElementHandles);
	return SetSelection(NativeHandles, InSelectionOptions);
}

bool UTypedElementSelectionSet::AllowSelectionModifiers(const FScriptTypedElementHandle& InElementHandle) const
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return AllowSelectionModifiers(NativeHandle);
}


FScriptTypedElementHandle UTypedElementSelectionSet::GetSelectionElement(const FScriptTypedElementHandle& InElementHandle, const ETypedElementSelectionMethod InSelectionMethod) const
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return FScriptTypedElementHandle();
	}

	return ElementList->GetRegistry()->CreateScriptHandle(GetSelectionElement(NativeHandle, InSelectionMethod).GetId());
}

TArray<FScriptTypedElementHandle> UTypedElementSelectionSet::K2_GetSelectedElementHandles(const TSubclassOf<UInterface> InBaseInterfaceType) const
{
	TArray<FTypedElementHandle> NativeHandles = GetSelectedElementHandles(InBaseInterfaceType);
	return TypedElementUtil::ConvertToScriptElementArray(NativeHandles, ElementList->GetRegistry());
}

