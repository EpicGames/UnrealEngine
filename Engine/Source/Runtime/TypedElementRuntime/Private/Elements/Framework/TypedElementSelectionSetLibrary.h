// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementListProxy.h"
#include "Elements/Framework/TypedElementSelectionSet.h"

#include "CoreMinimal.h"
#include "UObject/Stack.h"

#include "TypedElementSelectionSetLibrary.generated.h"

namespace UE::TypedElementFramework::Private
{
	// Return true if the selection set is invalid
	inline bool CheckForInvalidSelectionSet(UTypedElementSelectionSet* SelectionSet)
	{
		if (!SelectionSet)
		{
			FFrame::KismetExecutionMessage(TEXT("SelectionSet is null."), ELogVerbosity::Error);
		}

		return true;
	}
}

/**
 * Library of functions for the scripting of Typed Elements that use both a selection set and a element list
 * 
 * Note: These functions should only be used for scripting purposes only as they come at higher performance cost then their non script implementation
 */
UCLASS()
class UTypedElementSelectionSetLibrary : public UObject
{
	GENERATED_BODY()
	
public:
	/**
	 * Attempt to select the given elements.
	 * @return True if the selection was changed, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|Selection", meta=(ScriptMethod))
	static bool SelectElementsFromList(UTypedElementSelectionSet* SelectionSet, const FScriptTypedElementListProxy ElementList, const FTypedElementSelectionOptions SelectionOptions)
	{
		if (!UE::TypedElementFramework::Private::CheckForInvalidSelectionSet(SelectionSet))
		{
			return false;
		}

		FTypedElementListPtr NativeList = UE::TypedElementFramework::ConvertToNativeTypedElementList(ElementList.GetElementList());
		if (!NativeList)
		{
			FFrame::KismetExecutionMessage(TEXT("ElementList is in a invalid state."), ELogVerbosity::Error);
			return false;
		}

		return SelectionSet->SelectElements(NativeList.ToSharedRef(), SelectionOptions);
	}

	/**
	 * Attempt to deselect the given elements.
	 * @return True if the selection was changed, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|Selection", meta=(ScriptMethod))
	static bool DeselectElementsFromList(UTypedElementSelectionSet* SelectionSet, const FScriptTypedElementListProxy ElementList, const FTypedElementSelectionOptions SelectionOptions)
	{
		if (!UE::TypedElementFramework::Private::CheckForInvalidSelectionSet(SelectionSet))
		{
			return false;
		}

		FTypedElementListPtr NativeList = UE::TypedElementFramework::ConvertToNativeTypedElementList(ElementList.GetElementList());
		if (!NativeList)
		{
			FFrame::KismetExecutionMessage(TEXT("ElementList is in a invalid state."), ELogVerbosity::Error);
			return false;
		}

		return SelectionSet->DeselectElements(NativeList.ToSharedRef(), SelectionOptions);
	}

	/**
	 * Attempt to make the selection the given elements.
	 * @note Equivalent to calling ClearSelection then SelectElements, but happens in a single batch.
	 * @return True if the selection was changed, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|Selection", meta=(ScriptMethod))
	static bool SetSelectionFromList(UTypedElementSelectionSet* SelectionSet, const FScriptTypedElementListProxy ElementList, const FTypedElementSelectionOptions SelectionOptions)
	{
		if (!UE::TypedElementFramework::Private::CheckForInvalidSelectionSet(SelectionSet))
		{
			return false;
		}

		FTypedElementListPtr NativeList = UE::TypedElementFramework::ConvertToNativeTypedElementList(ElementList.GetElementList());
		return NativeList
			? SelectionSet->SetSelection(NativeList.ToSharedRef(), SelectionOptions)
			: SelectionSet->ClearSelection(SelectionOptions);
	}

	/**
	 * Get a normalized version of this selection set that can be used to perform operations like gizmo manipulation, deletion, copying, etc.
	 * This will do things like expand out groups, and resolve any parent<->child elements so that duplication operations aren't performed on both the parent and the child.
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|Selection", meta=(ScriptMethod))
	static FScriptTypedElementListProxy GetNormalizedSelection(UTypedElementSelectionSet* SelectionSet, const FTypedElementSelectionNormalizationOptions NormalizationOptions)
	{
		if (!UE::TypedElementFramework::Private::CheckForInvalidSelectionSet(SelectionSet))
		{
			return FScriptTypedElementListProxy();
		}

		FScriptTypedElementListPtr NormalizedSelection = UE::TypedElementFramework::ConvertToScriptTypedElementList(SelectionSet->GetNormalizedSelection(NormalizationOptions));
		return NormalizedSelection ? NormalizedSelection.ToSharedRef() : FScriptTypedElementListProxy();
	}

	/**
	 * Get a normalized version of the given element list that can be used to perform operations like gizmo manipulation, deletion, copying, etc.
	 * This will do things like expand out groups, and resolve any parent<->child elements so that duplication operations aren't performed on both the parent and the child.
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|Selection", meta=(ScriptMethod))
	static FScriptTypedElementListProxy GetNormalizedElementList(UTypedElementSelectionSet* SelectionSet, const FScriptTypedElementListProxy ElementList, const FTypedElementSelectionNormalizationOptions NormalizationOptions)
	{
		if (!UE::TypedElementFramework::Private::CheckForInvalidSelectionSet(SelectionSet))
		{
			return FScriptTypedElementListProxy();
		}

		FTypedElementListPtr NativeList = UE::TypedElementFramework::ConvertToNativeTypedElementList(ElementList.GetElementList());
		return NativeList ?  UE::TypedElementFramework::ConvertToScriptTypedElementList(SelectionSet->GetNormalizedElementList(NativeList.ToSharedRef(), NormalizationOptions)).ToSharedRef() : FScriptTypedElementListProxy();
	}
};
