// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Interfaces/TypedElementAssetDataInterface.h"

#include "UObject/Stack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementAssetDataInterface)

TArray<FAssetData> ITypedElementAssetDataInterface::GetAllReferencedAssetDatas(const FTypedElementHandle& InElementHandle, const FTypedElementAssetDataReferencedOptions& InOptions)
{
	TArray<FAssetData> AssetDatas;

	FAssetData ElementAssetData = GetAssetData(InElementHandle);
	if (ElementAssetData.IsValid())
	{
		AssetDatas.Emplace(ElementAssetData);
	}

	return AssetDatas;
}

FAssetData ITypedElementAssetDataInterface::GetAssetData(const FTypedElementHandle& InElementHandle)
{
	return FAssetData();
}

TArray<FAssetData> ITypedElementAssetDataInterface::GetAllReferencedAssetDatas(const FScriptTypedElementHandle& InElementHandle)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return {};
	}

	return GetAllReferencedAssetDatas(NativeHandle);
}

FAssetData ITypedElementAssetDataInterface::GetAssetData(const FScriptTypedElementHandle& InElementHandle)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return FAssetData();
	}

	return GetAssetData(NativeHandle);
}

