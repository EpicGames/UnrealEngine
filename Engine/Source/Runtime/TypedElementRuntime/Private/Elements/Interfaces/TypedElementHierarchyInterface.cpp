// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Interfaces/TypedElementHierarchyInterface.h"

#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementUtil.h"

#include "UObject/Stack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementHierarchyInterface)

FScriptTypedElementHandle ITypedElementHierarchyInterface::GetParentElement(const FScriptTypedElementHandle& InElementHandle, const bool bAllowCreate)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return FScriptTypedElementHandle();
	}
	
	return GetRegistry().CreateScriptHandle(GetParentElement(NativeHandle, bAllowCreate).GetId());
}

void ITypedElementHierarchyInterface::GetChildElements(const FScriptTypedElementHandle& InElementHandle, TArray<FScriptTypedElementHandle>& OutElementHandles, const bool bAllowCreate)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return;
	}

	TArray<FTypedElementHandle> NativeChildHandles;
	GetChildElements(NativeHandle, NativeChildHandles, bAllowCreate);

	OutElementHandles = TypedElementUtil::ConvertToScriptElementArray(NativeChildHandles, &GetRegistry());
}

class UTypedElementRegistry& ITypedElementHierarchyInterface::GetRegistry() const
{
	return *UTypedElementRegistry::GetInstance();
}


