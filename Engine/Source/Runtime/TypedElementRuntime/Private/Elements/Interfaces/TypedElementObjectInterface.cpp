// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Interfaces/TypedElementObjectInterface.h"

#include "Elements/Framework/TypedElementRegistry.h"

#include "UObject/Stack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementObjectInterface)

UObject* ITypedElementObjectInterface::GetObject(const FTypedElementHandle& InElementHandle)
{
	return nullptr;
}

UClass* ITypedElementObjectInterface::GetObjectClass(const FTypedElementHandle& InElementHandle)
{
	UObject* HandleAsObject = GetObject(InElementHandle);
	return HandleAsObject ? HandleAsObject->GetClass() : nullptr;
}

UObject* ITypedElementObjectInterface::GetObject(const FScriptTypedElementHandle& InElementHandle)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return nullptr;
	}

	return GetObject(NativeHandle);
}

UClass* ITypedElementObjectInterface::GetObjectClass(const FScriptTypedElementHandle& InElementHandle)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return nullptr;
	}

	return GetObjectClass(NativeHandle);
}


