// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Interfaces/TypedElementPrimitiveCustomDataInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementPrimitiveCustomDataInterface)

void ITypedElementPrimitiveCustomDataInterface::SetCustomData(const FScriptTypedElementHandle& InElementHandle, const TArray<float>& CustomDataFloats,  bool bMarkRenderStateDirty)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return;
	}

	SetCustomData(NativeHandle, MakeArrayView(CustomDataFloats.GetData(), CustomDataFloats.Num()), bMarkRenderStateDirty);
	
}

void ITypedElementPrimitiveCustomDataInterface::SetCustomDataValue(const FScriptTypedElementHandle& InElementHandle, int32 CustomDataIndex, float CustomDataValue, bool bMarkRenderStateDirty)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return;
	}
	
	SetCustomDataValue(NativeHandle, CustomDataIndex, CustomDataValue, bMarkRenderStateDirty);
}