// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/Interfaces/TypedElementSelectionInterface.h"

#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementUtil.h"

#include "UObject/Stack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypedElementSelectionInterface)

bool ITypedElementSelectionInterface::IsElementSelected(const FTypedElementHandle& InElementHandle, const FTypedElementListConstPtr& InSelectionSet, const FTypedElementIsSelectedOptions& InSelectionOptions)
{
	return InSelectionSet && InSelectionSet->Contains(InElementHandle);
}

bool ITypedElementSelectionInterface::SelectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	return InSelectionSet && InSelectionSet->Add(InElementHandle);
}

bool ITypedElementSelectionInterface::DeselectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	return InSelectionSet && InSelectionSet->Remove(InElementHandle);
}

bool ITypedElementSelectionInterface::IsElementSelected(const FScriptTypedElementHandle& InElementHandle, const FScriptTypedElementListProxy InSelectionSet, const FTypedElementIsSelectedOptions& InSelectionOptions)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	FTypedElementListPtr NativeList = UE::TypedElementFramework::ConvertToNativeTypedElementList(InSelectionSet.GetElementList());
	if (!NativeList)
	{
		FFrame::KismetExecutionMessage(TEXT("InSelectionSet is in a invalid state."), ELogVerbosity::Error);
		return false;
	}

	return IsElementSelected(NativeHandle, NativeList, InSelectionOptions);
}

bool ITypedElementSelectionInterface::CanSelectElement(const FScriptTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions& InSelectionOptions)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return CanSelectElement(NativeHandle, InSelectionOptions);
}

bool ITypedElementSelectionInterface::CanDeselectElement(const FScriptTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions& InSelectionOptions)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	return CanDeselectElement(NativeHandle, InSelectionOptions);
}

bool ITypedElementSelectionInterface::SelectElement(const FScriptTypedElementHandle& InElementHandle, FScriptTypedElementListProxy InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	FTypedElementListPtr NativeList = UE::TypedElementFramework::ConvertToNativeTypedElementList(InSelectionSet.GetElementList());
	if (!NativeList)
	{
		FFrame::KismetExecutionMessage(TEXT("InSelectionSet is in a invalid state."), ELogVerbosity::Error);
		return false;
	}

	return SelectElement(NativeHandle, NativeList, InSelectionOptions);
}

bool ITypedElementSelectionInterface::DeselectElement(const FScriptTypedElementHandle& InElementHandle, FScriptTypedElementListProxy InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	FTypedElementListPtr NativeList = UE::TypedElementFramework::ConvertToNativeTypedElementList(InSelectionSet.GetElementList());
	if (!NativeList)
	{
		FFrame::KismetExecutionMessage(TEXT("InSelectionSet is in a invalid state."), ELogVerbosity::Error);
		return false;
	}

	return DeselectElement(NativeHandle, NativeList, InSelectionOptions);
}

bool ITypedElementSelectionInterface::AllowSelectionModifiers(const FScriptTypedElementHandle& InElementHandle, const FScriptTypedElementListProxy InSelectionSet)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return false;
	}

	FTypedElementListPtr NativeList = UE::TypedElementFramework::ConvertToNativeTypedElementList(InSelectionSet.GetElementList());
	if (!NativeList)
	{
		FFrame::KismetExecutionMessage(TEXT("InSelectionSet is in a invalid state."), ELogVerbosity::Error);
		return false;
	}

	return AllowSelectionModifiers(NativeHandle, NativeList);
}

FScriptTypedElementHandle ITypedElementSelectionInterface::GetSelectionElement(const FScriptTypedElementHandle& InElementHandle, const FScriptTypedElementListProxy InCurrentSelection, const ETypedElementSelectionMethod InSelectionMethod)
{
	FTypedElementHandle NativeHandle = InElementHandle.GetTypedElementHandle();
	if (!NativeHandle)
	{
		FFrame::KismetExecutionMessage(TEXT("InElementHandle is not a valid handle."), ELogVerbosity::Error);
		return {};
	}

	FTypedElementListPtr NativeList = UE::TypedElementFramework::ConvertToNativeTypedElementList(InCurrentSelection.GetElementList());
	if (!NativeList)
	{
		FFrame::KismetExecutionMessage(TEXT("InCurrentSelection is in a invalid state."), ELogVerbosity::Error);
		return {};
	}

	return NativeList->GetRegistry()->CreateScriptHandle(GetSelectionElement(NativeHandle, NativeList, InSelectionMethod).GetId());
}

