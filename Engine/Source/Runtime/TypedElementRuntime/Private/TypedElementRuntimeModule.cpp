// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

class FTypedElementRuntimeModule : public FDefaultModuleImpl
{
public:
	virtual void StartupModule() override
	{
		FModuleManager::Get().LoadModuleChecked("TypedElementFramework");
	}
};

IMPLEMENT_MODULE(FTypedElementRuntimeModule, TypedElementRuntime)
