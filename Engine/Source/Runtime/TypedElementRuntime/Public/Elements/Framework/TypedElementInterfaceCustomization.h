// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Elements/Framework/TypedElementLimits.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

/**
 * Non-templated base class for the interface customization registry.
 */
class FTypedElementInterfaceCustomizationRegistryBase
{
public:
	virtual ~FTypedElementInterfaceCustomizationRegistryBase() = default;

protected:
	/**
	 * Given an element name, attempt to get its registered type ID from the global registry.
	 * @return The registered type ID, or 0 if the element name is not registered.
	 */
	TYPEDELEMENTRUNTIME_API FTypedHandleTypeId GetElementTypeIdFromName(const FName InElementTypeName) const;

	/**
	 * Given an element name, attempt to get its registered type ID from the global registry.
	 * @return The registered type ID, or asserts if the element name is not registered.
	 */
	TYPEDELEMENTRUNTIME_API FTypedHandleTypeId GetElementTypeIdFromNameChecked(const FName InElementTypeName) const;
};

/**
 * Utility to register and retrieve interface customizations for a given type.
 */
template <typename CustomizationBaseType, typename DefaultCustomizationType = CustomizationBaseType>
class TTypedElementInterfaceCustomizationRegistry : public FTypedElementInterfaceCustomizationRegistryBase
{
public:
	template <typename... TDefaultArgs>
	TTypedElementInterfaceCustomizationRegistry(TDefaultArgs&&... DefaultArgs)
		: DefaultInterfaceCustomization(MakeUnique<DefaultCustomizationType>(Forward<TDefaultArgs>(DefaultArgs)...))
	{
	}

	virtual ~TTypedElementInterfaceCustomizationRegistry() = default;

	TTypedElementInterfaceCustomizationRegistry(const TTypedElementInterfaceCustomizationRegistry&) = delete;
	TTypedElementInterfaceCustomizationRegistry& operator=(const TTypedElementInterfaceCustomizationRegistry&) = delete;

	TTypedElementInterfaceCustomizationRegistry(TTypedElementInterfaceCustomizationRegistry&&) = delete;
	TTypedElementInterfaceCustomizationRegistry& operator=(TTypedElementInterfaceCustomizationRegistry&&) = delete;

	/**
	 * Set the default interface customization instance.
	 */
	void SetDefaultInterfaceCustomization(TUniquePtr<CustomizationBaseType>&& InInterfaceCustomization)
	{
		checkf(InInterfaceCustomization, TEXT("Default interface customization cannot be null!"));
		DefaultInterfaceCustomization = MoveTemp(InInterfaceCustomization);
	}

	/**
	 * Return the default interface customization instance.
	 */
	CustomizationBaseType* GetDefaultInterfaceCustomization() const
	{
		return DefaultInterfaceCustomization.Get();
	}

	/**
	 * Register an interface customization for the given element type.
	 */
	void RegisterInterfaceCustomizationByTypeName(const FName InElementTypeName, TUniquePtr<CustomizationBaseType>&& InInterfaceCustomization)
	{
		RegisterInterfaceCustomizationByTypeId(GetElementTypeIdFromNameChecked(InElementTypeName), MoveTemp(InInterfaceCustomization));
	}

	/**
	 * Register an interface customization for the given element type.
	 */
	void RegisterInterfaceCustomizationByTypeId(const FTypedHandleTypeId InElementTypeId, TUniquePtr<CustomizationBaseType>&& InInterfaceCustomization)
	{
		RegisteredInterfaceCustomizations[InElementTypeId - 1] = MoveTemp(InInterfaceCustomization);
	}

	/**
	 * Unregister an interface customization for the given element type.
	 */
	void UnregisterInterfaceCustomizationByTypeName(const FName InElementTypeName)
	{
		UnregisterInterfaceCustomizationByTypeId(GetElementTypeIdFromNameChecked(InElementTypeName));
	}

	/**
	 * Unregister an interface customization for the given element type.
	 */
	void UnregisterInterfaceCustomizationByTypeId(const FTypedHandleTypeId InElementTypeId)
	{
		RegisteredInterfaceCustomizations[InElementTypeId - 1].Reset();
	}

	/**
	 * Get the interface customization for the given element type.
	 * @note If bAllowFallback is true, then this will return the default interface customization if no override is present, otherwise it will return null.
	 */
	CustomizationBaseType* GetInterfaceCustomizationByTypeName(const FName InElementTypeName, const bool bAllowFallback = true) const
	{
		return GetInterfaceCustomizationByTypeId(GetElementTypeIdFromNameChecked(InElementTypeName), bAllowFallback);
	}

	/**
	 * Get the interface customization for the given element type.
	 * @note If bAllowFallback is true, then this will return the default interface customization if no override is present, otherwise it will return null.
	 */
	CustomizationBaseType* GetInterfaceCustomizationByTypeId(const FTypedHandleTypeId InElementTypeId, const bool bAllowFallback = true) const
	{
		CustomizationBaseType* InterfaceCustomization = RegisteredInterfaceCustomizations[InElementTypeId - 1].Get();
		return InterfaceCustomization
			? InterfaceCustomization
			: bAllowFallback 
				? DefaultInterfaceCustomization.Get()
				: nullptr;
	}

private:
	/** Default interface customization, used if no type-specific override is present. */
	TUniquePtr<CustomizationBaseType> DefaultInterfaceCustomization;

	/** Array of registered interface customizations, indexed by ElementTypeId-1. */
	TUniquePtr<CustomizationBaseType> RegisteredInterfaceCustomizations[TypedHandleMaxTypeId - 1];
};
