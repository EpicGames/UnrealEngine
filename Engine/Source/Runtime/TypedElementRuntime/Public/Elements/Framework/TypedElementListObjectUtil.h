// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/TypedElementListFwd.h"
#include "HAL/Platform.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

class UClass;
class UObject;

namespace TypedElementListObjectUtil
{

/**
 * Test whether there are any objects in the given list of elements.
 */
TYPEDELEMENTRUNTIME_API bool HasObjects(FTypedElementListConstRef InElementList, const UClass* InRequiredClass = nullptr);

/**
 * Test whether there are any objects in the given list of elements.
 */
template <typename RequiredClassType>
bool HasObjects(FTypedElementListConstRef InElementList)
{
	return HasObjects(InElementList, RequiredClassType::StaticClass());
}

/**
 * Count the number of objects in the given list of elements.
 */
TYPEDELEMENTRUNTIME_API int32 CountObjects(FTypedElementListConstRef InElementList, const UClass* InRequiredClass = nullptr);

/**
 * Count the number of objects in the given list of elements.
 */
template <typename RequiredClassType>
int32 CountObjects(FTypedElementListConstRef InElementList)
{
	return CountObjects(InElementList, RequiredClassType::StaticClass());
}

/**
 * Enumerate the objects from the given list of elements.
 * @note Return true from the callback to continue enumeration.
 */
TYPEDELEMENTRUNTIME_API void ForEachObject(FTypedElementListConstRef InElementList, TFunctionRef<bool(UObject*)> InCallback, const UClass* InRequiredClass = nullptr);

/**
 * Enumerate the objects from the given list of elements.
 * @note Return true from the callback to continue enumeration.
 */
template <typename RequiredClassType>
void ForEachObject(FTypedElementListConstRef InElementList, TFunctionRef<bool(RequiredClassType*)> InCallback)
{
	ForEachObject(InElementList, [&InCallback](UObject* InObject)
	{
		return InCallback(CastChecked<RequiredClassType>(InObject));
	}, RequiredClassType::StaticClass());
}

/**
 * Get the array of objects from the given list of elements.
 */
TYPEDELEMENTRUNTIME_API TArray<UObject*> GetObjects(FTypedElementListConstRef InElementList, const UClass* InRequiredClass = nullptr);

/**
 * Get the array of objects from the given list of elements.
 */
template <typename RequiredClassType>
TArray<RequiredClassType*> GetObjects(FTypedElementListConstRef InElementList)
{
	TArray<RequiredClassType*> SelectedObjects;
	SelectedObjects.Reserve(InElementList->Num());

	ForEachObject<RequiredClassType>(InElementList, [&SelectedObjects](RequiredClassType* InObject)
	{
		SelectedObjects.Add(InObject);
		return true;
	});

	return SelectedObjects;
}

/**
 * Get the first object of the given type from the given list of elements.
 */
TYPEDELEMENTRUNTIME_API UObject* GetTopObject(FTypedElementListConstRef InElementList, const UClass* InRequiredClass = nullptr);

/**
 * Get the first object of the given type from the given list of elements.
 */
template <typename RequiredClassType>
RequiredClassType* GetTopObject(FTypedElementListConstRef InElementList)
{
	return Cast<RequiredClassType>(GetTopObject(InElementList, RequiredClassType::StaticClass()));
}

/**
 * Get the last object of the given type from the given list of elements.
 */
TYPEDELEMENTRUNTIME_API UObject* GetBottomObject(FTypedElementListConstRef InElementList, const UClass* InRequiredClass = nullptr);

/**
 * Get the last object of the given type from the given list of elements.
 */
template <typename RequiredClassType>
RequiredClassType* GetBottomObject(FTypedElementListConstRef InElementList)
{
	return Cast<RequiredClassType>(GetBottomObject(InElementList, RequiredClassType::StaticClass()));
}

/**
 * Test if there are any objects of the exact class in the given list of elements (a quick test using the class counter, skipping derived types).
 */
TYPEDELEMENTRUNTIME_API bool HasObjectsOfExactClass(FTypedElementListConstRef InElementList, const UClass* InClass);

/**
 * Test if there are any objects of the exact class in the given list of elements (a quick test using the class counter, skipping derived types).
 */
template <typename ClassType>
bool HasObjectsOfExactClass(FTypedElementListConstRef InElementList)
{
	return HasObjectsOfExactClass(InElementList, ClassType::StaticClass());
}

/**
 * Count the number of objects of the exact class in the given list of elements (a quick test using the class counter, skipping derived types).
 */
TYPEDELEMENTRUNTIME_API int32 CountObjectsOfExactClass(FTypedElementListConstRef InElementList, const UClass* InClass);

/**
 * Count the number of objects of the exact class in the given list of elements (a quick test using the class counter, skipping derived types).
 */
template <typename ClassType>
bool CountObjectsOfExactClass(FTypedElementListConstRef InElementList)
{
	return CountObjectsOfExactClass(InElementList, ClassType::StaticClass());
}

/**
 * Enumerate the classes of the objects in the given list of elements.
 * @note Return true from the callback to continue enumeration.
 */
TYPEDELEMENTRUNTIME_API void ForEachObjectClass(FTypedElementListConstRef InElementList, TFunctionRef<bool(UClass*)> InCallback);

} // namespace TypedElementListObjectUtil
