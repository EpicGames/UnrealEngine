// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Delegates/Delegate.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Framework/TypedElementInterfaceCustomization.h"
#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/TypedElementListFwd.h"
#include "Elements/Framework/TypedElementListObjectUtil.h"
#include "Elements/Interfaces/TypedElementSelectionInterface.h"
#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "TypedElementSelectionSet.generated.h"

class FArchive;
class UClass;
class UInterface;
class UTypedElementSelectionSet;
struct FFrame;

USTRUCT(BlueprintType)
struct FTypedElementSelectionNormalizationOptions
{
	GENERATED_BODY()

public:
	FTypedElementSelectionNormalizationOptions& SetExpandGroups(const bool InExpandGroups) { bExpandGroups = InExpandGroups; return *this; }
	bool ExpandGroups() const { return bExpandGroups; }

	FTypedElementSelectionNormalizationOptions& SetFollowAttachment(const bool InFollowAttachment) { bFollowAttachment = InFollowAttachment; return *this; }
	bool FollowAttachment() const { return bFollowAttachment; }

	// Set the selection set name that will be passed into the selection column in TEDS (if it is enabled)
	FTypedElementSelectionNormalizationOptions& SetNameForTEDSIntegration(const FName& InTEDSIntegrationSelectionSetName) { TEDSIntegrationSelectionSetName = InTEDSIntegrationSelectionSetName; return *this; }
	FName GetNameForTEDSIntegration() const { return TEDSIntegrationSelectionSetName; }
	
private:
	UPROPERTY(BlueprintReadWrite, Category="TypedElementInterfaces|Selection|NormalizationOptions", meta=(AllowPrivateAccess=true))
	bool bExpandGroups = false;

	UPROPERTY(BlueprintReadWrite, Category="TypedElementInterfaces|Selection|NormalizationOptions", meta=(AllowPrivateAccess=true))
	bool bFollowAttachment = false;

	FName TEDSIntegrationSelectionSetName = FName();
};

/**
 * Customization type used to allow asset editors (such as the level editor) to override the base behavior of element selection,
 * by injecting extra pre/post selection logic around the call into the selection interface for an element type.
 */
class FTypedElementSelectionCustomization
{
public:
	virtual ~FTypedElementSelectionCustomization() = default;

	//~ See ITypedElementSelectionInterface for API docs
	virtual bool IsElementSelected(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InSelectionSet, const FTypedElementIsSelectedOptions& InSelectionOptions) { return InElementSelectionHandle.IsElementSelected(InSelectionSet, InSelectionOptions); }
	virtual bool CanSelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions) { return InElementSelectionHandle.CanSelectElement(InSelectionOptions); }
	virtual bool CanDeselectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, const FTypedElementSelectionOptions& InSelectionOptions) { return InElementSelectionHandle.CanDeselectElement(InSelectionOptions); }
	virtual bool SelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) { return InElementSelectionHandle.SelectElement(InSelectionSet, InSelectionOptions); }
	virtual bool DeselectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) { return InElementSelectionHandle.DeselectElement(InSelectionSet, InSelectionOptions); }
	virtual bool AllowSelectionModifiers(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InSelectionSet) { return InElementSelectionHandle.AllowSelectionModifiers(InSelectionSet); }
	virtual FTypedElementHandle GetSelectionElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InCurrentSelection, const ETypedElementSelectionMethod InSelectionMethod) { return InElementSelectionHandle.GetSelectionElement(InCurrentSelection, InSelectionMethod); }
	TYPEDELEMENTRUNTIME_API virtual void GetNormalizedElements(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListConstRef InSelectionSet, const FTypedElementSelectionNormalizationOptions& InNormalizationOptions, FTypedElementListRef OutNormalizedElements);
};

/**
 * Utility to hold a typed element handle and its associated selection interface and selection customization.
 */
struct FTypedElementSelectionSetElement
{
public:
	FTypedElementSelectionSetElement() = default;

	FTypedElementSelectionSetElement(TTypedElement<ITypedElementSelectionInterface> InElementSelectionHandle, FTypedElementListPtr InElementList, FTypedElementSelectionCustomization* InSelectionCustomization)
		: ElementSelectionHandle(MoveTemp(InElementSelectionHandle))
		, ElementList(InElementList)
		, SelectionCustomization(InSelectionCustomization)
	{
	}

	FTypedElementSelectionSetElement(const FTypedElementSelectionSetElement&) = default;
	FTypedElementSelectionSetElement& operator=(const FTypedElementSelectionSetElement&) = default;

	FTypedElementSelectionSetElement(FTypedElementSelectionSetElement&&) = default;
	FTypedElementSelectionSetElement& operator=(FTypedElementSelectionSetElement&&) = default;

	inline explicit operator bool() const
	{
		return IsSet();
	}

	inline bool IsSet() const
	{
		return ElementSelectionHandle.IsSet()
			&& ElementList
			&& SelectionCustomization;
	}

	//~ See ITypedElementSelectionInterface for API docs
	bool IsElementSelected(const FTypedElementIsSelectedOptions& InSelectionOptions) const { return SelectionCustomization->IsElementSelected(ElementSelectionHandle, ElementList.ToSharedRef(), InSelectionOptions); }
	bool CanSelectElement(const FTypedElementSelectionOptions& InSelectionOptions) const { return SelectionCustomization->CanSelectElement(ElementSelectionHandle, InSelectionOptions); }
	bool CanDeselectElement(const FTypedElementSelectionOptions& InSelectionOptions) const { return SelectionCustomization->CanDeselectElement(ElementSelectionHandle, InSelectionOptions); }
	bool SelectElement(const FTypedElementSelectionOptions& InSelectionOptions) const { return SelectionCustomization->SelectElement(ElementSelectionHandle, ElementList.ToSharedRef(), InSelectionOptions); }
	bool DeselectElement(const FTypedElementSelectionOptions& InSelectionOptions) const { return SelectionCustomization->DeselectElement(ElementSelectionHandle, ElementList.ToSharedRef(), InSelectionOptions); }
	bool AllowSelectionModifiers() const { return SelectionCustomization->AllowSelectionModifiers(ElementSelectionHandle, ElementList.ToSharedRef()); }
	FTypedElementHandle GetSelectionElement(const ETypedElementSelectionMethod InSelectionMethod) const { return SelectionCustomization->GetSelectionElement(ElementSelectionHandle, ElementList.ToSharedRef(), InSelectionMethod); }
	void GetNormalizedElements(const FTypedElementSelectionNormalizationOptions& InNormalizationOptions, FTypedElementListRef OutNormalizedElements) const { return SelectionCustomization->GetNormalizedElements(ElementSelectionHandle, ElementList.ToSharedRef(), InNormalizationOptions, OutNormalizedElements); }

private:
	TTypedElement<ITypedElementSelectionInterface> ElementSelectionHandle;
	FTypedElementListPtr ElementList;
	FTypedElementSelectionCustomization* SelectionCustomization = nullptr;
};

USTRUCT(BlueprintType)
struct FTypedElementSelectionSetState
{
	GENERATED_BODY()

	friend class UTypedElementSelectionSet;

	FTypedElementSelectionSetState() = default;

	FTypedElementSelectionSetState(FTypedElementSelectionSetState&&) = default;
	FTypedElementSelectionSetState& operator=(FTypedElementSelectionSetState&&) = default;

	FTypedElementSelectionSetState(const FTypedElementSelectionSetState& InOther)
		: CreatedFromSelectionSet(InOther.CreatedFromSelectionSet)
	{
		TransactedElements.Reserve(InOther.TransactedElements.Num());
		for (const TUniquePtr<ITypedElementTransactedElement>& OtherTransactedElement : InOther.TransactedElements)
		{
			TransactedElements.Emplace(OtherTransactedElement->Clone());
		}
	}

	FTypedElementSelectionSetState& operator=(const FTypedElementSelectionSetState& InOther)
	{
		if (&InOther != this)
		{
			CreatedFromSelectionSet = InOther.CreatedFromSelectionSet;

			TransactedElements.Reset(InOther.TransactedElements.Num());
			for (const TUniquePtr<ITypedElementTransactedElement>& OtherTransactedElement : InOther.TransactedElements)
			{
				TransactedElements.Emplace(OtherTransactedElement->Clone());
			}
		}
		return *this;
	}

private:
	UPROPERTY()
	TWeakObjectPtr<const UTypedElementSelectionSet> CreatedFromSelectionSet;

	TArray<TUniquePtr<ITypedElementTransactedElement>> TransactedElements;
};

/**
 * A wrapper around an element list that ensures mutation goes via the selection 
 * interfaces, as well as providing some utilities for batching operations.
 */
UCLASS(Transient, BlueprintType, meta=(DontUseGenericSpawnObject="True"), MinimalAPI)
class UTypedElementSelectionSet : public UObject, public TTypedElementInterfaceCustomizationRegistry<FTypedElementSelectionCustomization>
{
	GENERATED_BODY()

public:
	TYPEDELEMENTRUNTIME_API UTypedElementSelectionSet();

	//~ UObject interface
#if WITH_EDITOR
	TYPEDELEMENTRUNTIME_API virtual void PreEditUndo() override;
	TYPEDELEMENTRUNTIME_API virtual void PostEditUndo() override;
	TYPEDELEMENTRUNTIME_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;
#endif	// WITH_EDITOR
	TYPEDELEMENTRUNTIME_API virtual void BeginDestroy() override;
	TYPEDELEMENTRUNTIME_API virtual void Serialize(FArchive& Ar) override;

	/**
	 * Test to see whether the given element is currently considered selected.
	 */
	TYPEDELEMENTRUNTIME_API bool IsElementSelected(const FTypedElementHandle& InElementHandle, const FTypedElementIsSelectedOptions InSelectionOptions) const;

	/**
	 * Test to see whether the given element can be selected.
	 */
	TYPEDELEMENTRUNTIME_API bool CanSelectElement(const FTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions InSelectionOptions) const;

	/**
	 * Test to see whether the given element can be deselected.
	 */
	TYPEDELEMENTRUNTIME_API bool CanDeselectElement(const FTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions InSelectionOptions) const;

	/**
	 * Attempt to select the given element.
	 * @return True if the selection was changed, false otherwise.
	 */
	TYPEDELEMENTRUNTIME_API bool SelectElement(const FTypedElementHandle& InElementHandle, FTypedElementSelectionOptions InSelectionOptions);

	/**
	 * Attempt to select the given elements.
	 * @return True if the selection was changed, false otherwise.
	 */
	TYPEDELEMENTRUNTIME_API bool SelectElements(const TArray<FTypedElementHandle>& InElementHandles, const FTypedElementSelectionOptions InSelectionOptions);
	TYPEDELEMENTRUNTIME_API bool SelectElements(TArrayView<const FTypedElementHandle> InElementHandles, const FTypedElementSelectionOptions InSelectionOptions);
	TYPEDELEMENTRUNTIME_API bool SelectElements(FTypedElementListConstRef InElementList, const FTypedElementSelectionOptions InSelectionOptions);

	/**
	 * Attempt to deselect the given element.
	 * @return True if the selection was changed, false otherwise.
	 */
	TYPEDELEMENTRUNTIME_API bool DeselectElement(const FTypedElementHandle& InElementHandle, FTypedElementSelectionOptions InSelectionOptions);

	/**
	 * Attempt to deselect the given elements.
	 * @return True if the selection was changed, false otherwise.
	 */
	TYPEDELEMENTRUNTIME_API bool DeselectElements(const TArray<FTypedElementHandle>& InElementHandles, const FTypedElementSelectionOptions InSelectionOptions);
	TYPEDELEMENTRUNTIME_API bool DeselectElements(TArrayView<const FTypedElementHandle> InElementHandles, const FTypedElementSelectionOptions InSelectionOptions);
	TYPEDELEMENTRUNTIME_API bool DeselectElements(FTypedElementListConstRef InElementList, const FTypedElementSelectionOptions InSelectionOptions);

	/**
	 * Clear the current selection.
	 * @return True if the selection was changed, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|Selection")
	TYPEDELEMENTRUNTIME_API bool ClearSelection(const FTypedElementSelectionOptions InSelectionOptions);

	/**
	 * Attempt to make the selection the given elements.
	 * @note Equivalent to calling ClearSelection then SelectElements, but happens in a single batch.
	 * @return True if the selection was changed, false otherwise.
	 */
	TYPEDELEMENTRUNTIME_API bool SetSelection(const TArray<FTypedElementHandle>& InElementHandles, const FTypedElementSelectionOptions InSelectionOptions);
	TYPEDELEMENTRUNTIME_API bool SetSelection(TArrayView<const FTypedElementHandle> InElementHandles, const FTypedElementSelectionOptions InSelectionOptions);
	TYPEDELEMENTRUNTIME_API bool SetSelection(FTypedElementListConstRef InElementList, const FTypedElementSelectionOptions InSelectionOptions);

	/**
	 * Test to see whether selection modifiers (Ctrl or Shift) are allowed while selecting this element.
	 */
	TYPEDELEMENTRUNTIME_API bool AllowSelectionModifiers(const FTypedElementHandle& InElementHandle) const;

	/**
	 * Sets the name to use for teds integration so that it can be reapplied on undo/redo
	 */
	TYPEDELEMENTRUNTIME_API void SetNameForTedsIntegration(const FName InNameForIntegration);
	
	TYPEDELEMENTRUNTIME_API FName GetNameForTedsIntegration() const;

	/**
	 * Given an element, return the element that should actually perform a selection operation.
	 */
	TYPEDELEMENTRUNTIME_API FTypedElementHandle GetSelectionElement(const FTypedElementHandle& InElementHandle, const ETypedElementSelectionMethod InSelectionMethod) const;

	/**
	 * Get a normalized version of this selection set that can be used to perform operations like gizmo manipulation, deletion, copying, etc.
	 * This will do things like expand out groups, and resolve any parent<->child elements so that duplication operations aren't performed on both the parent and the child.
	 */
	TYPEDELEMENTRUNTIME_API FTypedElementListRef GetNormalizedSelection(const FTypedElementSelectionNormalizationOptions InNormalizationOptions) const;
	TYPEDELEMENTRUNTIME_API void GetNormalizedSelection(const FTypedElementSelectionNormalizationOptions& InNormalizationOptions, FTypedElementListRef OutNormalizedElements) const;

	/**
	 * Get a normalized version of the given element list that can be used to perform operations like gizmo manipulation, deletion, copying, etc.
	 * This will do things like expand out groups, and resolve any parent<->child elements so that duplication operations aren't performed on both the parent and the child.
	 */
	TYPEDELEMENTRUNTIME_API FTypedElementListRef GetNormalizedElementList(FTypedElementListConstRef InElementList, const FTypedElementSelectionNormalizationOptions InNormalizationOptions) const;
	TYPEDELEMENTRUNTIME_API void GetNormalizedElementList(FTypedElementListConstRef InElementList, const FTypedElementSelectionNormalizationOptions& InNormalizationOptions, FTypedElementListRef OutNormalizedElements) const;

	/**
	 * Get the number of selected elements.
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|Selection")
	int32 GetNumSelectedElements() const
	{
		return ElementList->Num();
	}

	/**
	 * Test whether there selected elements, optionally filtering to elements that implement the given interface.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="TypedElementFramework|Selection")
	bool HasSelectedElements(const TSubclassOf<UInterface> InBaseInterfaceType = {}) const
	{
		return ElementList->HasElements(InBaseInterfaceType);
	}

	/**
	 * Count the number of selected elements, optionally filtering to elements that implement the given interface.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="TypedElementFramework|Selection")
	int32 CountSelectedElements(const TSubclassOf<UInterface> InBaseInterfaceType = {}) const
	{
		return ElementList->CountElements(InBaseInterfaceType);
	}

	/**
	 * Get the handle of every selected element, optionally filtering to elements that implement the given interface.
	 */
	TArray<FTypedElementHandle> GetSelectedElementHandles(const TSubclassOf<UInterface> InBaseInterfaceType = {}) const
	{
		return ElementList->GetElementHandles(InBaseInterfaceType);
	}

	/**
	 * Get the handle of every selected element, optionally filtering to elements that implement the given interface.
	 */
	template <typename ArrayAllocator>
	void GetSelectedElementHandles(TArray<FTypedElementHandle, ArrayAllocator>& OutArray, const TSubclassOf<UInterface>& InBaseInterfaceType = {}) const
	{
		ElementList->GetElementHandles(OutArray, InBaseInterfaceType);
	}

	/**
	 * Enumerate the handle of every selected element, optionally filtering to elements that implement the given interface.
	 * @note Return true from the callback to continue enumeration.
	 */
	void ForEachSelectedElementHandle(TFunctionRef<bool(const FTypedElementHandle&)> InCallback, const TSubclassOf<UInterface>& InBaseInterfaceType = {}) const
	{
		ElementList->ForEachElementHandle(InCallback, InBaseInterfaceType);
	}

	/**
	 * Enumerate the selected elements that implement the given interface.
	 * @note Return true from the callback to continue enumeration.
	 */
	template <typename BaseInterfaceType>
	void ForEachSelectedElement(TFunctionRef<bool(const TTypedElement<BaseInterfaceType>&)> InCallback) const
	{
		ElementList->ForEachElement<BaseInterfaceType>(InCallback);
	}

	/**
	 * Get the first selected element implementing the given interface.
	 */
	template <typename BaseInterfaceType>
	TTypedElement<BaseInterfaceType> GetTopSelectedElement() const
	{
		return ElementList->GetTopElement<BaseInterfaceType>();
	}

	/**
	 * Get the first element that implement the given interface and pass the predicate
	 * @Predicate A function that should return true when the element is desirable
	 */
	template <typename BaseInterfaceType>
	TTypedElement<BaseInterfaceType> GetTopSelectedElement(TFunctionRef<bool (const TTypedElement<BaseInterfaceType>&)> Predicate) const
	{
		return ElementList->GetTopElement<BaseInterfaceType>(Predicate);
	}

	/**
	 * Get the last selected element implementing the given interface.
	 */
	template <typename BaseInterfaceType>
	TTypedElement<BaseInterfaceType> GetBottomSelectedElement() const
	{
		return ElementList->GetBottomElement<BaseInterfaceType>();
	}

	/**
	 * Get the last element that implement the given interface and pass the predicate.
	 * @Predicate A function that return should true when the element is desirable
	 */
	template <typename BaseInterfaceType>
	TTypedElement<BaseInterfaceType> GetBottomSelectedElement(TFunctionRef<bool (const TTypedElement<BaseInterfaceType>&)> Predicate) const
	{
		return ElementList->GetBottomElement<BaseInterfaceType>(Predicate);
	}

	/**
	 * Test whether there are any selected objects.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="TypedElementFramework|Selection")
	bool HasSelectedObjects(const UClass* InRequiredClass = nullptr) const
	{
		return TypedElementListObjectUtil::HasObjects(ElementList.ToSharedRef(), InRequiredClass);
	}

	/**
	 * Test whether there are any selected objects.
	 */
	template <typename RequiredClassType>
	bool HasSelectedObjects() const
	{
		return TypedElementListObjectUtil::HasObjects<RequiredClassType>(ElementList.ToSharedRef());
	}

	/**
	 * Count the number of selected objects.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="TypedElementFramework|Selection")
	int32 CountSelectedObjects(const UClass* InRequiredClass = nullptr) const
	{
		return TypedElementListObjectUtil::CountObjects(ElementList.ToSharedRef(), InRequiredClass);
	}

	/**
	 * Count the number of selected objects.
	 */
	template <typename RequiredClassType>
	int32 CountSelectedObjects() const
	{
		return TypedElementListObjectUtil::CountObjects<RequiredClassType>(ElementList.ToSharedRef());
	}

	/**
	 * Get the array of selected objects from the currently selected elements.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="TypedElementFramework|Selection")
	TArray<UObject*> GetSelectedObjects(const UClass* InRequiredClass = nullptr) const
	{
		return TypedElementListObjectUtil::GetObjects(ElementList.ToSharedRef(), InRequiredClass);
	}

	/**
	 * Get the array of selected objects from the currently selected elements.
	 */
	template <typename RequiredClassType>
	TArray<RequiredClassType*> GetSelectedObjects() const
	{
		return TypedElementListObjectUtil::GetObjects<RequiredClassType>(ElementList.ToSharedRef());
	}

	/**
	 * Enumerate the selected objects from the currently selected elements.
	 * @note Return true from the callback to continue enumeration.
	 */
	void ForEachSelectedObject(TFunctionRef<bool(UObject*)> InCallback, const UClass* InRequiredClass = nullptr) const
	{
		TypedElementListObjectUtil::ForEachObject(ElementList.ToSharedRef(), InCallback, InRequiredClass);
	}

	/**
	 * Enumerate the selected objects from the currently selected elements.
	 * @note Return true from the callback to continue enumeration.
	 */
	template <typename RequiredClassType>
	void ForEachSelectedObject(TFunctionRef<bool(RequiredClassType*)> InCallback) const
	{
		TypedElementListObjectUtil::ForEachObject<RequiredClassType>(ElementList.ToSharedRef(), InCallback);
	}

	/**
	 * Get the first selected object of the given type.
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|Selection")
	UObject* GetTopSelectedObject(const UClass* InRequiredClass = nullptr) const
	{
		return TypedElementListObjectUtil::GetTopObject(ElementList.ToSharedRef(), InRequiredClass);
	}

	/**
	 * Get the first selected object of the given type.
	 */
	template <typename RequiredClassType>
	RequiredClassType* GetTopSelectedObject() const
	{
		return TypedElementListObjectUtil::GetTopObject<RequiredClassType>(ElementList.ToSharedRef());
	}

	/**
	 * Get the last selected object of the given type.
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|Selection")
	UObject* GetBottomSelectedObject(const UClass* InRequiredClass = nullptr) const
	{
		return TypedElementListObjectUtil::GetBottomObject(ElementList.ToSharedRef(), InRequiredClass);
	}

	/**
	 * Get the last selected object of the given type.
	 */
	template <typename RequiredClassType>
	RequiredClassType* GetBottomSelectedObject() const
	{
		return TypedElementListObjectUtil::GetBottomObject<RequiredClassType>(ElementList.ToSharedRef());
	}

	/**
	 * Access the delegate that is invoked whenever the underlying element list is potentially about to change.
	 * @note This may be called even if no actual change happens, so may be called multiple times without a corresponding OnChanged notification.
	 */
	DECLARE_EVENT_OneParam(UTypedElementSelectionSet, FOnPreChange, const UTypedElementSelectionSet* /*InElementSelectionSet*/);
	FOnPreChange& OnPreChange()
	{
		return OnPreChangeDelegate;
	}

	/**
	 * Access the delegate that is invoked whenever the underlying element list has been changed.
	 * @note This is called automatically at the end of each frame, but can also be manually invoked by NotifyPendingChanges.
	 */
	DECLARE_EVENT_OneParam(UTypedElementSelectionSet, FOnChanged, const UTypedElementSelectionSet* /*InElementSelectionSet*/);
	FOnChanged& OnChanged()
	{
		return OnChangedDelegate;
	}

	/**
	 * Invoke the delegate called whenever the underlying element list has been changed.
	 */
	void NotifyPendingChanges()
	{
		ElementList->NotifyPendingChanges();
	}

	/**
	 * Get a scoped object that when destroyed it clear a pending change notification without emitting the notification if it happened during its lifecycle.
	 */
	FTypedElementList::FScopedClearNewPendingChange GetScopedClearNewPendingChange()
	{
		return ElementList->GetScopedClearNewPendingChange();
	}

	/**
	 * Access the interface to allow external systems (such as USelection) to receive immediate sync notifications as the underlying element list is changed.
	 * This exists purely as a bridging mechanism and shouldn't be relied on for new code. It is lazily created as needed.
	 */
	FTypedElementList::FLegacySync& Legacy_GetElementListSync()
	{
		return ElementList->Legacy_GetSync();
	}

	/**
	 * Access the interface to allow external systems (such as USelection) to receive immediate sync notifications as the underlying element list is changed.
	 * This exists purely as a bridging mechanism and shouldn't be relied on for new code. This will return null if no legacy sync has been created for this instance.
	 */
	FTypedElementList::FLegacySync* Legacy_GetElementListSyncPtr()
	{
		return ElementList->Legacy_GetSyncPtr();
	}

	/**
	 * Get the underlying element list holding the selection state.
	 */
	FTypedElementListConstRef GetElementList() const
	{
		return ElementList.ToSharedRef();
	}

	/**
	 * Serializes the current selection set. 
	 * The calling code is responsible for storing any state information. The selection set can be returned to a prior state using RestoreSelectionState.
	 *
	 * @returns the current state of the selection set.
	 */
	UFUNCTION(BlueprintPure, Category = "TypedElementFramework|Selection")
	TYPEDELEMENTRUNTIME_API FTypedElementSelectionSetState GetCurrentSelectionState() const;

	/**
	 * Restores the selection set from the given state.
	 * The calling code is responsible for managing any undo state.
	 */
	UFUNCTION(BlueprintCallable, Category = "TypedElementFramework|Selection")
	TYPEDELEMENTRUNTIME_API void RestoreSelectionState(const FTypedElementSelectionSetState& InSelectionState);


	/*
	 * Script Api
	 */

	/**
	 * Test to see whether the given element is currently considered selected.
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|Selection")
	TYPEDELEMENTRUNTIME_API bool IsElementSelected(const FScriptTypedElementHandle& InElementHandle, const FTypedElementIsSelectedOptions InSelectionOptions) const;

	/**
	 * Test to see whether the given element can be selected.
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|Selection")
	TYPEDELEMENTRUNTIME_API bool CanSelectElement(const FScriptTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions InSelectionOptions) const;

	/**
	 * Test to see whether the given element can be deselected.
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|Selection")
	TYPEDELEMENTRUNTIME_API bool CanDeselectElement(const FScriptTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions InSelectionOptions) const;

	/**
	 * Attempt to select the given element.
	 * @return True if the selection was changed, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|Selection")
	TYPEDELEMENTRUNTIME_API bool SelectElement(const FScriptTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions InSelectionOptions);

	/**
	 * Attempt to select the given elements.
	 * @return True if the selection was changed, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|Selection")
	TYPEDELEMENTRUNTIME_API bool SelectElements(const TArray<FScriptTypedElementHandle>& InElementHandles, const FTypedElementSelectionOptions InSelectionOptions);

	/**
	 * Attempt to deselect the given element.
	 * @return True if the selection was changed, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|Selection")
	TYPEDELEMENTRUNTIME_API bool DeselectElement(const FScriptTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions InSelectionOptions);

	/**
	 * Attempt to deselect the given elements.
	 * @return True if the selection was changed, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|Selection")
	TYPEDELEMENTRUNTIME_API bool DeselectElements(const TArray<FScriptTypedElementHandle>& InElementHandles, const FTypedElementSelectionOptions InSelectionOptions);

	/**
	 * Attempt to make the selection the given elements.
	 * @note Equivalent to calling ClearSelection then SelectElements, but happens in a single batch.
	 * @return True if the selection was changed, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementFramework|Selection")
	TYPEDELEMENTRUNTIME_API bool SetSelection(const TArray<FScriptTypedElementHandle>& InElementHandles, const FTypedElementSelectionOptions InSelectionOptions);

	/**
	 * Test to see whether selection modifiers (Ctrl or Shift) are allowed while selecting this element.
	 */
	UFUNCTION(BlueprintPure, Category="TypedElementFramework|Selection")
	TYPEDELEMENTRUNTIME_API bool AllowSelectionModifiers(const FScriptTypedElementHandle& InElementHandle) const;

	/**
	 * Given an element, return the element that should actually perform a selection operation.
	 */
	UFUNCTION(BlueprintPure, Category = "TypedElementFramework|Selection")
	TYPEDELEMENTRUNTIME_API FScriptTypedElementHandle GetSelectionElement(const FScriptTypedElementHandle& InElementHandle, const ETypedElementSelectionMethod InSelectionMethod) const;


	/**
	 * Get the handle of every selected element, optionally filtering to elements that implement the given interface.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, DisplayName="Get Selected Element Handles", Category="TypedElementFramework|Selection", meta=(ScriptName="GetSelectedElementHandles"))
	TYPEDELEMENTRUNTIME_API TArray<FScriptTypedElementHandle> K2_GetSelectedElementHandles(const TSubclassOf<UInterface> InBaseInterfaceType = {}) const;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnPreChangeDynamic, const UTypedElementSelectionSet*, SelectionSet);

	/** Delegate that is invoked whenever the underlying element list is potentially about to change. */
	UPROPERTY(BlueprintAssignable, Category = "TypedElementFramework|Selection")
	FOnPreChangeDynamic OnPreSelectionChange;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnChangeDynamic, const UTypedElementSelectionSet*, SelectionSet);

	/** Delegate that is invoked whenever the underlying element list has been changed. */
	UPROPERTY(BlueprintAssignable, Category = "TypedElementFramework|Selection")
	FOnChangeDynamic OnSelectionChange;

private:
	/**
	 * Attempt to resolve the selection interface and selection customization for the given element, if any.
	 */
	TYPEDELEMENTRUNTIME_API FTypedElementSelectionSetElement ResolveSelectionSetElement(const FTypedElementHandle& InElementHandle) const;

	/**
	 * Update the selection from a replacement request.
	 */
	TYPEDELEMENTRUNTIME_API void OnElementReplaced(TArrayView<const TTuple<FTypedElementHandle, FTypedElementHandle>> InReplacedElements);

	/**
	 * Force a selection update if an element internal state changes.
	 */
	TYPEDELEMENTRUNTIME_API void OnElementUpdated(TArrayView<const FTypedElementHandle> InUpdatedElements);

	/**
	 * Proxy the internal OnPreChange event from the underlying element list.
	 */
	TYPEDELEMENTRUNTIME_API void OnElementListPreChange(const FTypedElementList& InElementList);

	/**
	 * Proxy the internal OnChanged event from the underlying element list.
	 */
	TYPEDELEMENTRUNTIME_API void OnElementListChanged(const FTypedElementList& InElementList);

	/** Underlying element list holding the selection state. */
	FTypedElementListPtr ElementList;

	/** Delegate that is invoked whenever the underlying element list is potentially about to change. */
	FOnPreChange OnPreChangeDelegate;

	/** Delegate that is invoked whenever the underlying element list has been changed. */
	FOnChanged OnChangedDelegate;

	/** Set when we are currently restoring the selection state (eg, from an undo/redo) */
	bool bIsRestoringState = false;

	/**
	 * Set between a PreEditUndo->PostEditUndo call.
	 * Initially holds the state cached during PreEditUndo which is saved during Serialize, 
	 * and is then replaced with the state loaded from Serialize to be applied in PostEditUndo.
	 */
	TUniquePtr<FTypedElementSelectionSetState> PendingUndoRedoState;

	FName ListNameForTedsIntegration;
};
