// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "TypedElementAssetDataInterface.generated.h"

class UObject;
struct FFrame;

USTRUCT(BlueprintType)
struct FTypedElementAssetDataReferencedOptions
{
	GENERATED_BODY()

public:
	FTypedElementAssetDataReferencedOptions& SetOnlyTopLevelAsset(const bool InTopLevelAsset) { bOnlyTopLevelAsset = InTopLevelAsset; return *this; }
	bool OnlyTopLevelAsset() const { return bOnlyTopLevelAsset; }

private:
	UPROPERTY(BlueprintReadWrite, Category = "TypedElementInterfaces|AssetData|GetReferencedOptions", meta = (AllowPrivateAccess = true))
	bool bOnlyTopLevelAsset = false;
};

UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UTypedElementAssetDataInterface : public UInterface
{
	GENERATED_BODY()
};

class ITypedElementAssetDataInterface
{
	GENERATED_BODY()

public:
	/**
	 * Returns any asset datas for content objects referenced by handle.
	 * If the given handle itself has valid asset data, it should be returned as the last element of the array.
	 *
	 * @returns An array of valid asset datas.
	 */
	TYPEDELEMENTRUNTIME_API virtual TArray<FAssetData> GetAllReferencedAssetDatas(const FTypedElementHandle& InElementHandle, const FTypedElementAssetDataReferencedOptions& InOptions = FTypedElementAssetDataReferencedOptions());

	/**
	 * Returns the asset data for the given handle, if it exists.
	 */
	TYPEDELEMENTRUNTIME_API virtual FAssetData GetAssetData(const FTypedElementHandle& InElementHandle);

	/**
	 * Script Api
	 */

	/**
	 * Returns any asset datas for content objects referenced by handle.
	 * If the given handle itself has valid asset data, it should be returned as the last element of the array.
	 *
	 * @returns An array of valid asset datas.
	 */
	UFUNCTION(BlueprintCallable, Category = "TypedElementInterfaces|AssetData")
	TYPEDELEMENTRUNTIME_API virtual TArray<FAssetData> GetAllReferencedAssetDatas(const FScriptTypedElementHandle& InElementHandle);

	/**
	 * Returns the asset data for the given handle, if it exists.
	 */
	UFUNCTION(BlueprintCallable, Category = "TypedElementInterfaces|AssetData")
	TYPEDELEMENTRUNTIME_API virtual FAssetData GetAssetData(const FScriptTypedElementHandle& InElementHandle);
};

template <>
struct TTypedElement<ITypedElementAssetDataInterface> : public TTypedElementBase<ITypedElementAssetDataInterface>
{
	TArray<FAssetData> GetAllReferencedAssetDatas(const FTypedElementAssetDataReferencedOptions& InOptions = FTypedElementAssetDataReferencedOptions()) const { return InterfacePtr->GetAllReferencedAssetDatas(*this, InOptions); }
	FAssetData GetAssetData() const { return InterfacePtr->GetAssetData(*this); }
};
