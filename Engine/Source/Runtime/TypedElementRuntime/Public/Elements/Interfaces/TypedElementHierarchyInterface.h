// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "TypedElementHierarchyInterface.generated.h"

class UObject;
struct FFrame;

UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UTypedElementHierarchyInterface : public UInterface
{
	GENERATED_BODY()
};

class ITypedElementHierarchyInterface
{
	GENERATED_BODY()

public:
	/**
	 * Get the logical parent of this element, if any.
	 * eg) A component might return its actor, or a static mesh instance might return its ISM component.
	 */
	virtual FTypedElementHandle GetParentElement(const FTypedElementHandle& InElementHandle, const bool bAllowCreate = true) { return FTypedElementHandle(); }

	/**
	 * Get the logical children of this element, if any.
	 * eg) An actor might return its component, or an ISM component might return its static mesh instances.
	 *
	 * @note Appends to OutElementHandles.
	 */
	virtual void GetChildElements(const FTypedElementHandle& InElementHandle, TArray<FTypedElementHandle>& OutElementHandles, const bool bAllowCreate = true) {}

	/**
	 * Script Api
	 */

	/**
	 * Get the logical parent of this element, if any.
	 * eg) A component might return its actor, or a static mesh instance might return its ISM component.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|Hierarchy")
	TYPEDELEMENTRUNTIME_API virtual FScriptTypedElementHandle GetParentElement(const FScriptTypedElementHandle& InElementHandle, const bool bAllowCreate = true);

	/**
	 * Get the logical children of this element, if any.
	 * eg) An actor might return its component, or an ISM component might return its static mesh instances.
	 *
	 * @note Appends to OutElementHandles.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|Hierarchy")
	TYPEDELEMENTRUNTIME_API virtual void GetChildElements(const FScriptTypedElementHandle& InElementHandle, TArray<FScriptTypedElementHandle>& OutElementHandles, const bool bAllowCreate = true);

private:
	
	/**
	 * Return the registry used when creating new elements.
	 */
	TYPEDELEMENTRUNTIME_API virtual class UTypedElementRegistry& GetRegistry() const;
};

template <>
struct TTypedElement<ITypedElementHierarchyInterface> : public TTypedElementBase<ITypedElementHierarchyInterface>
{
	FTypedElementHandle GetParentElement(const bool bAllowCreate = true) const { return InterfacePtr->GetParentElement(*this, bAllowCreate); }
	void GetChildElements(TArray<FTypedElementHandle>& OutElementHandles, const bool bAllowCreate = true) const { return InterfacePtr->GetChildElements(*this, OutElementHandles, bAllowCreate); }
};
