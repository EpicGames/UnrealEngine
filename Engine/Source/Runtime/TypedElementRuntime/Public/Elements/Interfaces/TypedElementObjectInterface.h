// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementHandle.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Interface.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "TypedElementObjectInterface.generated.h"

class UClass;
struct FFrame;

UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UTypedElementObjectInterface : public UInterface
{
	GENERATED_BODY()
};

class ITypedElementObjectInterface
{
	GENERATED_BODY()

public:
	/**
	 * Get the object instance that this handle represents, if any.
	 */
	TYPEDELEMENTRUNTIME_API virtual UObject* GetObject(const FTypedElementHandle& InElementHandle);

	/**
	 * Gets the object instance's class that the handle represents, if any. 
	 */
	TYPEDELEMENTRUNTIME_API virtual UClass* GetObjectClass(const FTypedElementHandle& InElementHandle);

	/**
	 * Attempts to cast the given handle to another class.
	 *
	 * @returns the casted object if successful, otherwise nullptr.
	 */
	template <class CastTo>
	CastTo* GetObjectAs(const FTypedElementHandle& InElementHandle)
	{
		return Cast<CastTo>(GetObject(InElementHandle));
	}

	/**
	 * Attempts to cast the given handle to another class, if it can also be casted to TargetClass.
	 * This is intended for use in cases where the calling code may only need an actor pointer, but also wants to be sure it's a specific type of actor.
	 *
	 * @returns the casted object if successful, otherwise nullptr.
	 */
	template <class CastTo = UObject>
	CastTo* GetObjectAs(const FTypedElementHandle& InElementHandle, TSubclassOf<CastTo> TargetClass)
	{
		if (!TargetClass)
		{
			return nullptr;
		}

		UObject* ObjectPtr = GetObject(InElementHandle);
		if (ObjectPtr && ObjectPtr->IsA(TargetClass))
		{
			return (CastTo*)ObjectPtr;
		}

		return nullptr;
	}


	/**
	 * Script api
	 */

	/**
	 * Get the object instance that this handle represents, if any.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|Object")
	TYPEDELEMENTRUNTIME_API virtual UObject* GetObject(const FScriptTypedElementHandle& InElementHandle);

	/**
	 * Gets the object instance's class that the handle represents, if any. 
	 */
	UFUNCTION(BlueprintCallable, Category = "TypedElementInterfaces|Object")
	TYPEDELEMENTRUNTIME_API virtual UClass* GetObjectClass(const FScriptTypedElementHandle& InElementHandle);
};

template <>
struct TTypedElement<ITypedElementObjectInterface> : public TTypedElementBase<ITypedElementObjectInterface>
{
	UObject* GetObject() const { return InterfacePtr->GetObject(*this); }
	UClass* GetObjectClass() const { return InterfacePtr->GetObjectClass(*this); }

	template <class CastTo>
	CastTo* GetObjectAs() const { return InterfacePtr->GetObjectAs<CastTo>(*this); }

	template <class CastTo>
	CastTo* GetObjectAs(TSubclassOf<CastTo> TargetClass) const { return InterfacePtr->GetObjectAs<CastTo>(*this, TargetClass); }
};
