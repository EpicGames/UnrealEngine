// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementHandle.h"
#include "UObject/Interface.h"

#include "TypedElementPrimitiveCustomDataInterface.generated.h"

UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UTypedElementPrimitiveCustomDataInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * An interface for use with the TypedElement Framework which exposes Primitive CustomData
 * for use within Materials.
 */
class ITypedElementPrimitiveCustomDataInterface
{
	GENERATED_BODY()

public:
	
	virtual void SetCustomData(const FTypedElementHandle& InElementHandle, TArrayView<const float> CustomDataFloats,  bool bMarkRenderStateDirty = false) {}
	virtual void SetCustomDataValue(const FTypedElementHandle& InElementHandle, int32 CustomDataIndex, float CustomDataValue, bool bMarkRenderStateDirty = false) {};

	//~ Scripting Interface

	// Sets all Primitive's CustomData values
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|CustomData")
	TYPEDELEMENTRUNTIME_API virtual void SetCustomData(const FScriptTypedElementHandle& InElementHandle, const TArray<float>& CustomDataFloats,  bool bMarkRenderStateDirty = false);

	// Sets a single Primitive's CustomData value
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|CustomData")
	TYPEDELEMENTRUNTIME_API virtual void SetCustomDataValue(const FScriptTypedElementHandle& InElementHandle, int32 CustomDataIndex, float CustomDataValue, bool bMarkRenderStateDirty = false);
};

template <>
struct TTypedElement<ITypedElementPrimitiveCustomDataInterface> : public TTypedElementBase<ITypedElementPrimitiveCustomDataInterface>
{
	void SetCustomData(TArrayView<float> CustomDataFloats,  bool bMarkRenderStateDirty = false) const { return InterfacePtr->SetCustomData(*this, CustomDataFloats, bMarkRenderStateDirty); }
	void SetCustomDataValue(int32 CustomDataIndex, float CustomDataValue, bool bMarkRenderStateDirty = false) { return InterfacePtr->SetCustomDataValue(*this, CustomDataIndex, CustomDataValue, bMarkRenderStateDirty); }
};
