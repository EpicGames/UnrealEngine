// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Framework/TypedElementId.h"
#include "Elements/Framework/TypedElementLimits.h"
#include "Elements/Framework/TypedElementListFwd.h"
#include "Elements/Framework/TypedElementListProxy.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Templates/UniquePtr.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "TypedElementSelectionInterface.generated.h"

class UObject;
struct FFrame;

UENUM()
enum class ETypedElementSelectionMethod : uint8
{
	/** Select the "primary" element (eg, a component would favor selecting its owner actor) */
	Primary,
	/** Select the "secondary" element (eg, a component would favor selecting itself) */
	Secondary,
	/** The "secondary" element is choosing to pass back up the chain (eg. an ISM was selected, but we want to traverse back to the parent element) */
	FromSecondary,
};

UENUM()
enum class ETypedElementChildInclusionMethod : uint8
{
	/** Do not include child elements */
	None,
	/** Include the immediate child elements, but do not recurse */
	Immediate,
	/** Include the immediate child elements, and recurse into their children too */
	Recursive,
};

USTRUCT(BlueprintType)
struct FTypedElementIsSelectedOptions
{
	GENERATED_BODY()

public:
	FTypedElementIsSelectedOptions& SetAllowIndirect(const bool InAllowIndirect) { bAllowIndirect = InAllowIndirect; return *this; }
	bool AllowIndirect() const { return bAllowIndirect; }

	// Set the selection set name that will be passed into the selection column in TEDS (if it is enabled)
	FTypedElementIsSelectedOptions& SetNameForTEDSIntegration(const FName& InTEDSIntegrationSelectionSetName) { TEDSIntegrationSelectionSetName = InTEDSIntegrationSelectionSetName; return *this; }
	FName GetNameForTEDSIntegration() const { return TEDSIntegrationSelectionSetName; }
	
private:
	UPROPERTY(BlueprintReadWrite, Category="TypedElementInterfaces|Selection|IsSelectedOptions", meta=(AllowPrivateAccess=true))
	bool bAllowIndirect = false;
	
	FName TEDSIntegrationSelectionSetName = FName();
};

USTRUCT(BlueprintType)
struct FTypedElementSelectionOptions
{
	GENERATED_BODY()

public:
	FTypedElementSelectionOptions& SetAllowHidden(const bool InAllowHidden) { bAllowHidden = InAllowHidden; return *this; }
	bool AllowHidden() const { return bAllowHidden; }

	FTypedElementSelectionOptions& SetAllowGroups(const bool InAllowGroups) { bAllowGroups = InAllowGroups; return *this; }
	bool AllowGroups() const { return bAllowGroups; }

	FTypedElementSelectionOptions& SetAllowLegacyNotifications(const bool InAllowLegacyNotifications) { bAllowLegacyNotifications = InAllowLegacyNotifications; return *this; }
	bool AllowLegacyNotifications() const { return bAllowLegacyNotifications; }

	FTypedElementSelectionOptions& SetWarnIfLocked(const bool InWarnIfLocked) { bWarnIfLocked = InWarnIfLocked; return *this; }
	bool WarnIfLocked() const { return bWarnIfLocked; }

	FTypedElementSelectionOptions& SetAllowSubRootSelection(const bool InAllowSubRootSelectioin) { bAllowSubRootSelection = InAllowSubRootSelectioin; return *this; }
	bool AllowSubRootSelection() const { return bAllowSubRootSelection; }

	FTypedElementSelectionOptions& SetChildElementInclusionMethod(const ETypedElementChildInclusionMethod InChildElementInclusionMethod) { ChildElementInclusionMethod = InChildElementInclusionMethod; return *this; }
	ETypedElementChildInclusionMethod GetChildElementInclusionMethod() const { return ChildElementInclusionMethod; }

	// Set the selection set name that will be passed into the selection column in TEDS (if it is enabled)
	FTypedElementSelectionOptions& SetNameForTEDSIntegration(const FName& InTEDSIntegrationSelectionSetName) { TEDSIntegrationSelectionSetName = InTEDSIntegrationSelectionSetName; return *this; }
	FName GetNameForTEDSIntegration() const { return TEDSIntegrationSelectionSetName; }
private:
	UPROPERTY(BlueprintReadWrite, Category="TypedElementInterfaces|Selection|SelectionOptions", meta=(AllowPrivateAccess=true))
	bool bAllowHidden = false;

	UPROPERTY(BlueprintReadWrite, Category="TypedElementInterfaces|Selection|SelectionOptions", meta=(AllowPrivateAccess=true))
	bool bAllowGroups = true;

	UPROPERTY(BlueprintReadWrite, Category="TypedElementInterfaces|Selection|SelectionOptions", meta=(AllowPrivateAccess=true))
	bool bAllowLegacyNotifications = true;

	UPROPERTY(BlueprintReadWrite, Category="TypedElementInterfaces|Selection|SelectionOptions", meta=(AllowPrivateAccess=true))
	bool bWarnIfLocked = false;

	UPROPERTY(BlueprintReadWrite, Category = "TypedElementInterfaces|Selection|SelectionOptions", meta = (AllowPrivateAccess = true))
	bool bAllowSubRootSelection = false;

	UPROPERTY(BlueprintReadWrite, Category="TypedElementInterfaces|Selection|SelectionOptions", meta=(AllowPrivateAccess=true))
	ETypedElementChildInclusionMethod ChildElementInclusionMethod = ETypedElementChildInclusionMethod::None;

	FName TEDSIntegrationSelectionSetName = FName();
};

class ITypedElementTransactedElement
{
public:
	virtual ~ITypedElementTransactedElement() = default;

	TUniquePtr<ITypedElementTransactedElement> Clone() const
	{
		TUniquePtr<ITypedElementTransactedElement> Cloned = CloneImpl();
		checkf(Cloned, TEXT("ITypedElementTransactedElement derived types must implement a valid Clone function!"));
		return Cloned;
	}

	FTypedElementHandle GetElement() const
	{
		return GetElementImpl();
	}

	FTypedHandleTypeId GetElementType() const
	{
		return TypeId;
	}

	void SetElement(const FTypedElementHandle& InElementHandle)
	{
		SetElementType(InElementHandle.GetId().GetTypeId());
		SetElementImpl(InElementHandle);
	}

	void SetElementType(const FTypedHandleTypeId InTypeId)
	{
		TypeId = InTypeId;
	}

	void Serialize(FArchive& InArchive)
	{
		checkf(!InArchive.IsPersistent(), TEXT("ITypedElementTransactedElement can only be serialized by transient archives!"));
		SerializeImpl(InArchive);
	}

protected:
	virtual TUniquePtr<ITypedElementTransactedElement> CloneImpl() const = 0;
	virtual FTypedElementHandle GetElementImpl() const = 0;
	virtual void SetElementImpl(const FTypedElementHandle& InElementHandle) = 0;
	virtual void SerializeImpl(FArchive& InArchive) = 0;

private:
	FTypedHandleTypeId TypeId = 0;
};

UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UTypedElementSelectionInterface : public UInterface
{
	GENERATED_BODY()
};

class ITypedElementSelectionInterface
{
	GENERATED_BODY()

public:
	/**
	 * Test to see whether the given element is currently considered selected.
	 */
	TYPEDELEMENTRUNTIME_API virtual bool IsElementSelected(const FTypedElementHandle& InElementHandle, const FTypedElementListConstPtr& SelectionSetPtr, const FTypedElementIsSelectedOptions& InSelectionOptions);

	/**
	 * Test to see whether the given element can be selected.
	 */
	virtual bool CanSelectElement(const FTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions& InSelectionOptions) { return true; }

	/**
	 * Test to see whether the given element can be deselected.
	 */
	virtual bool CanDeselectElement(const FTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions& InSelectionOptions) { return true; }

	/**
	 * Attempt to select the given element.
	 * @return True if the selection was changed, false otherwise.
	 */
	TYPEDELEMENTRUNTIME_API virtual bool SelectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions);

	/**
	 * Attempt to deselect the given element.
	 * @return True if the selection was changed, false otherwise.
	 */
	TYPEDELEMENTRUNTIME_API virtual bool DeselectElement(const FTypedElementHandle& InElementHandle, const FTypedElementListPtr& InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions);

	/**
	 * Test to see whether selection modifiers (Ctrl or Shift) are allowed while selecting this element.
	 */
	virtual bool AllowSelectionModifiers(const FTypedElementHandle& InElementHandle, const FTypedElementListConstPtr& InSelectionSet) { return true; }

	/**
	 * Given an element, return the element that should actually perform a selection operation.
	 */
	virtual FTypedElementHandle GetSelectionElement(const FTypedElementHandle& InElementHandle, const FTypedElementListConstPtr& InCurrentSelection, const ETypedElementSelectionMethod InSelectionMethod) { return InElementHandle; }

	/**
	 * Test to see whether the given element prevents the selection set state from being transacted for undo/redo (eg, if the element belongs to a PIE instance).
	 */
	virtual bool ShouldPreventTransactions(const FTypedElementHandle& InElementHandle) { return false; }

	/**
	 * Create a transacted element instance that can be used to save the given element for undo/redo.
	 */
	TUniquePtr<ITypedElementTransactedElement> CreateTransactedElement(const FTypedElementHandle& InElementHandle)
	{
		TUniquePtr<ITypedElementTransactedElement> TransactedElement = CreateTransactedElementImpl();
		if (TransactedElement)
		{
			TransactedElement->SetElement(InElementHandle);
		}
		return TransactedElement;
	}

	/**
	 * Create a transacted element instance that can be used to load an element previously saved for undo/redo.
	 */
	TUniquePtr<ITypedElementTransactedElement> CreateTransactedElement(const FTypedHandleTypeId InTypeId)
	{
		TUniquePtr<ITypedElementTransactedElement> TransactedElement = CreateTransactedElementImpl();
		if (TransactedElement)
		{
			TransactedElement->SetElementType(InTypeId);
		}
		return TransactedElement;
	}


	/**
	 * Script Api
	 */

	/**
	 * Test to see whether the given element is currently considered selected.
	 */
	UFUNCTION(BlueprintCallable, Category = "TypedElementInterfaces|Selection")
	TYPEDELEMENTRUNTIME_API virtual bool IsElementSelected(const FScriptTypedElementHandle& InElementHandle, const FScriptTypedElementListProxy InSelectionSet, const FTypedElementIsSelectedOptions& InSelectionOptions);

	/**
	 * Test to see whether the given element can be selected.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|Selection")
	TYPEDELEMENTRUNTIME_API virtual bool CanSelectElement(const FScriptTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions& InSelectionOptions);

	/**
	 * Test to see whether the given element can be deselected.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|Selection")
	TYPEDELEMENTRUNTIME_API virtual bool CanDeselectElement(const FScriptTypedElementHandle& InElementHandle, const FTypedElementSelectionOptions& InSelectionOptions);

	/**
	 * Attempt to select the given element.
	 * @return True if the selection was changed, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|Selection")
	TYPEDELEMENTRUNTIME_API virtual bool SelectElement(const FScriptTypedElementHandle& InElementHandle, FScriptTypedElementListProxy InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions);

	/**
	 * Attempt to deselect the given element.
	 * @return True if the selection was changed, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|Selection")
	TYPEDELEMENTRUNTIME_API virtual bool DeselectElement(const FScriptTypedElementHandle& InElementHandle, FScriptTypedElementListProxy InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions);

	/**
	 * Test to see whether selection modifiers (Ctrl or Shift) are allowed while selecting this element.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|Selection")
	TYPEDELEMENTRUNTIME_API virtual bool AllowSelectionModifiers(const FScriptTypedElementHandle& InElementHandle, const FScriptTypedElementListProxy InSelectionSet);

	/**
	 * Given an element, return the element that should actually perform a selection operation.
	 */
	UFUNCTION(BlueprintCallable, Category="TypedElementInterfaces|Selection")
	TYPEDELEMENTRUNTIME_API virtual FScriptTypedElementHandle GetSelectionElement(const FScriptTypedElementHandle& InElementHandle, const FScriptTypedElementListProxy InCurrentSelection, const ETypedElementSelectionMethod InSelectionMethod);

protected:
	/**
	 * Create a transacted element instance that can be used to save/load elements of the implementation type for undo/redo.
	 * @note The instance returned from this function must have either SetElement or SetElementType called on it prior to being used.
	 */
	virtual TUniquePtr<ITypedElementTransactedElement> CreateTransactedElementImpl() { return nullptr; }
};

template <>
struct TTypedElement<ITypedElementSelectionInterface> : public TTypedElementBase<ITypedElementSelectionInterface>
{
	bool IsElementSelected(FTypedElementListConstRef InSelectionSet, const FTypedElementIsSelectedOptions& InSelectionOptions) const { return InterfacePtr->IsElementSelected(*this, InSelectionSet, InSelectionOptions); }
	bool CanSelectElement(const FTypedElementSelectionOptions& InSelectionOptions) const { return InterfacePtr->CanSelectElement(*this, InSelectionOptions); }
	bool CanDeselectElement(const FTypedElementSelectionOptions& InSelectionOptions) const { return InterfacePtr->CanDeselectElement(*this, InSelectionOptions); }
	bool SelectElement(FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) const { return InterfacePtr->SelectElement(*this, InSelectionSet, InSelectionOptions); }
	bool DeselectElement(FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) const { return InterfacePtr->DeselectElement(*this, InSelectionSet, InSelectionOptions); }
	bool AllowSelectionModifiers(FTypedElementListConstRef InSelectionSet) const { return InterfacePtr->AllowSelectionModifiers(*this, InSelectionSet); }
	FTypedElementHandle GetSelectionElement(FTypedElementListConstRef InCurrentSelection, const ETypedElementSelectionMethod InSelectionMethod) const { return InterfacePtr->GetSelectionElement(*this, InCurrentSelection, InSelectionMethod); }
	bool ShouldPreventTransactions() const { return InterfacePtr->ShouldPreventTransactions(*this); }
	TUniquePtr<ITypedElementTransactedElement> CreateTransactedElement() const { return InterfacePtr->CreateTransactedElement(*this); }
};
