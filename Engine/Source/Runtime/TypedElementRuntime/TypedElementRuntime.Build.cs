// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class TypedElementRuntime : ModuleRules
{
	public TypedElementRuntime(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"TypedElementFramework",
			}
		);
    }
}
